{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "InstancedActors",
	"Description": "",
	"Category": "Gameplay",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "http://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "InstancedActors",
			"Type": "Runtime"
		},
		{
			"Name": "InstancedActorsTestSuite",
			"Type": "UncookedOnly"
		},
		{
			"Name": "InstancedActorsEditor",
			"Type": "Editor"
		}
	],
	"Plugins": [
		{
			"Name": "MassGameplay",
			"Enabled": true
		},
		{
			"Name": "DataRegistry",
			"Enabled": true
		},
		{
			"Name": "GameFeatures",
			"Enabled": true
		}
	]
}