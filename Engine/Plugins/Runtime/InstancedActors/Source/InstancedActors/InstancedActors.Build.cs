// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class InstancedActors : ModuleRules
	{
		public InstancedActors(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"GameplayTags",
					"MassCore",
					"MassEntity",
					"MassCommon",
					"MassActors",
					"MassRepresentation",
					"MassSpawner",
					"MassLOD",
					"MassSmartObjects",
					"MassSignals",
					"DataRegistry",
					"DeveloperSettings",
					"NetCore",
					"GameFeatures"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}

			if (Target.Configuration != UnrealTargetConfiguration.Shipping)
			{
				PrivateDependencyModuleNames.AddRange(
					new string[] {
						"InputCore",
						"MassGameplayDebug"
					}
				);
			}

			SetupGameplayDebuggerSupport(Target);
			SetupIrisSupport(Target);
		}
	}
}
