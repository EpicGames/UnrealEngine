// Copyright Epic Games, Inc. All Rights Reserved.

#include "ClientInstancedActorsSpawnerSubsystem.h"
#include "InstancedActorsComponent.h"
#include "InstancedActorsDebug.h"
#include "MassEntitySubsystem.h"
#include "Engine/Level.h"
#include "InstancedActorsSettings.h"


//-----------------------------------------------------------------------------
// UClientInstancedActorsSpawnerSubsystem
//-----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(ClientInstancedActorsSpawnerSubsystem)
bool UClientInstancedActorsSpawnerSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	if (!Super::ShouldCreateSubsystem(Outer))
	{
		return false;
	}

	// @todo Add support for non-replay NM_Standalone where we should use UServerInstancedActorsSpawnerSubsystem for 
	// authoritative actor spawning.
	UWorld* World = Cast<UWorld>(Outer);
	return (World != nullptr && World->GetNetMode() == NM_Client);
}

ESpawnRequestStatus UClientInstancedActorsSpawnerSubsystem::SpawnActor(const FSpawnArgs& Args, FActorSpawnParameters& InOutSpawnParameters) const
{
	// UInstancedActorsVisualizationTrait should be setting all LOD representations to StaticMeshInstance to avoid
	// ever attempting to natively spawn an actor.
	// 
	// Instead we rely on UInstancedActorsComponent's, added dynamically to all actors on spawn in UServerInstancedActorsSpawnerSubsystem::SpawnActor, 
	// to replicate over and 'inject' the Actor into Mass in UInstancedActorsComponent::OnRep_InstanceHandle.
	//
	// UInstancedActorsVisualizationTrait also sets bForceActorRepresentationWhileAvailable on clients so that once the replicated 
	// actor is registered with Mass, we switch into Actor representation and stay there until Actor destruction is replicate,
	// whereupon we switch back to whatever the natural wanted representation is (ISMC)
	ensureMsgf(false, TEXT("UClientInstancedActorsSpawnerSubsystem::SpawnActor unexpectedly called on client where we shouldn't ever be trying to spawn new actors."));

	return ESpawnRequestStatus::Pending;
}

bool UClientInstancedActorsSpawnerSubsystem::ReleaseActorToPool(AActor* Actor)
{
	// As we set bForceActorRepresentationWhileAvailable on clients (see above), we should never be attempting to 
	// explicitly destroy actors from Mass. Rather, actor destruction should be happening via replication from server,
	// relying on UInstancedActorsComponent::EndPlay to clean up the actor reference in Mass.
	ensureMsgf(false, TEXT("UClientInstancedActorsSpawnerSubsystem::ReleaseActorToPool unexpectedly called on client where we shouldn't ever be trying to destroy actors."));

	return true;
}
