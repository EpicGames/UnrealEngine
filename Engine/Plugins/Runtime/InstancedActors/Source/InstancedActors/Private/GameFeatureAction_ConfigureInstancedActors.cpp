// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameFeatureAction_ConfigureInstancedActors.h"
#include "InstancedActorsSettings.h"
#include "InstancedActorsSubsystem.h"
#include "GameFeaturesSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(GameFeatureAction_ConfigureInstancedActors)


void UGameFeatureAction_ConfigureInstancedActors::OnGameFeatureActivating(FGameFeatureActivatingContext& Context)
{
	Super::OnGameFeatureActivating(Context);

	GetMutableDefault<UInstancedActorsProjectSettings>()->RegisterConfigOverride(*this, ConfigOverride);
}

void UGameFeatureAction_ConfigureInstancedActors::OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context)
{
	GetMutableDefault<UInstancedActorsProjectSettings>()->UnregisterConfigOverride(*this);

	Super::OnGameFeatureDeactivating(Context);
}
