// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerCategory_InstancedActors.h"

#if WITH_GAMEPLAY_DEBUGGER && WITH_INSTANCEDACTORS_DEBUG && WITH_MASSENTITY_DEBUG

#include "InstancedActorsDebug.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"
#include "InstancedActorsTypes.h"
#include "InstancedActorsSettingsTypes.h"

#include "GameplayDebuggerCategoryReplicator.h"
#include "HAL/IConsoleManager.h"
#include "MassActorSubsystem.h"
#include "MassAgentComponent.h"
#include "MassDebugger.h"
#include "MassDebuggerSubsystem.h"
#include "MassEntitySubsystem.h"
#include "MassEntityView.h"
#include "Engine/World.h"


namespace UE::InstancedActors::Debug
{
	FMassEntityHandle GetEntityFromActor(const AActor& Actor, const UMassAgentComponent*& OutMassAgentComponent)
	{
		FMassEntityHandle EntityHandle;
		if (const UMassAgentComponent* AgentComp = Actor.FindComponentByClass<UMassAgentComponent>())
		{
			EntityHandle = AgentComp->GetEntityHandle();
			OutMassAgentComponent = AgentComp;
		}
		else if (UMassActorSubsystem* ActorSubsystem = UWorld::GetSubsystem<UMassActorSubsystem>(Actor.GetWorld()))
		{
			EntityHandle = ActorSubsystem->GetEntityHandleFromActor(&Actor);
		}
		return EntityHandle;
	};
}

TArray<FAutoConsoleCommandWithWorld> FGameplayDebuggerCategory_InstancedActors::ConsoleCommands;
FGameplayDebuggerCategory_InstancedActors::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_InstancedActors::OnToggleDebugLocalIAMsBroadcast;


FGameplayDebuggerCategory_InstancedActors::FGameplayDebuggerCategory_InstancedActors()
{
	bShowOnlyWithDebugActor = false;

	OnEntitySelectedHandle = FMassDebugger::OnEntitySelectedDelegate.AddRaw(this, &FGameplayDebuggerCategory_InstancedActors::OnEntitySelected);

	ToggleDebugLocalIAMsInputIndex = GetNumInputHandlers();
	BindKeyPress(EKeys::L.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_InstancedActors::OnToggleDebugLocalIAMs, EGameplayDebuggerInputMode::Local);

	if (ConsoleCommands.Num() == 0)
	{
		ConsoleCommands.Emplace(TEXT("gdt.ia.ToggleDebugLocalIAMs"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleDebugLocalIAMsBroadcast.Broadcast(InWorld); }));
	}

	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleDebugLocalIAMsBroadcast, OnToggleDebugLocalIAMsBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleDebugLocalIAMs(); }})));
}

FGameplayDebuggerCategory_InstancedActors::~FGameplayDebuggerCategory_InstancedActors()
{
	FMassDebugger::OnEntitySelectedDelegate.Remove(OnEntitySelectedHandle);
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_InstancedActors::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_InstancedActors());
}

void FGameplayDebuggerCategory_InstancedActors::SetCachedEntity(const FMassEntityHandle Entity, const FMassEntityManager& EntityManager)
{
	if (CachedEntity != Entity)
	{
		FMassDebugger::SelectEntity(EntityManager, Entity);
	}
}

void FGameplayDebuggerCategory_InstancedActors::ClearCachedEntity()
{
	CachedEntity = FMassEntityHandle();
}

void FGameplayDebuggerCategory_InstancedActors::OnEntitySelected(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle)
{
	UWorld* World = EntityManager.GetWorld();
	if (World != GetWorldFromReplicator())
	{ 
		// ignore, this call is for a different world
		return;
	}

	AActor* BestActor = nullptr;
	if (EntityHandle.IsSet() && EntityManager.IsEntityValid(EntityHandle) && World)
	{
		if (const UMassActorSubsystem* ActorSubsystem = World->GetSubsystem<UMassActorSubsystem>())
		{
			BestActor = ActorSubsystem->GetActorFromHandle(EntityHandle);
		}
		if (BestActor == nullptr)
		{
			UInstancedActorsData* InstancedActorData = UInstancedActorsData::GetInstanceDataForEntity(EntityManager, EntityHandle);
			if (InstancedActorData)
			{
				BestActor = InstancedActorData->GetManager();
			}
		}
	}

	CachedEntity = EntityHandle;
	CachedDebugActor = BestActor;
	check(GetReplicator());
	GetReplicator()->SetDebugActor(BestActor);
}

void FGameplayDebuggerCategory_InstancedActors::OnToggleDebugLocalIAMs()
{
	// this code will only execute on locally-controlled categories (as per BindKeyPress's EGameplayDebuggerInputMode::Local
	// parameter). In such a case we don't want to toggle if we're also Auth (there's no client-server relationship here).
	if (IsCategoryAuth())
	{
		return;
	}

	ResetReplicatedData();
	bDebugLocalIAMs = !bDebugLocalIAMs;
	bAllowLocalDataCollection = bDebugLocalIAMs;

	const EGameplayDebuggerInputMode NewInputMode = bDebugLocalIAMs ? EGameplayDebuggerInputMode::Local : EGameplayDebuggerInputMode::Replicated;
	for (int32 HandlerIndex = 0; HandlerIndex < GetNumInputHandlers(); ++HandlerIndex)
	{
		if (HandlerIndex != ToggleDebugLocalIAMsInputIndex)
		{
			GetInputHandler(HandlerIndex).Mode = NewInputMode;
		}
	}

	CachedEntity.Reset();
}

void FGameplayDebuggerCategory_InstancedActors::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	if (bAllowLocalDataCollection)
	{
		ResetReplicatedData();
	}

	// we only want to display this if there are local/remote roles in play
	if (IsCategoryAuth() != IsCategoryLocal())
	{
		AddTextLine(FString::Printf(TEXT("Source: {yellow}%s{white}"), bDebugLocalIAMs ? TEXT("LOCAL") : TEXT("REMOTE")));
	}

	UWorld* World = GetDataWorld(OwnerPC, DebugActor);
	check(World);

	UMassEntitySubsystem* EntitySubsystem = UWorld::GetSubsystem<UMassEntitySubsystem>(World);
	if (EntitySubsystem == nullptr)
	{
		AddTextLine(FString::Printf(TEXT("{Red}EntitySubsystem instance is missing")));
		return;
	}
	FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();

	const UMassAgentComponent* AgentComp = nullptr;
	
	if (bAllowLocalDataCollection)
	{
		DebugActor = CachedDebugActor;
	}

	if (DebugActor)
	{
		if (!DebugActor->IsA<AInstancedActorsManager>())
		{
			const FMassEntityHandle EntityHandle = UE::InstancedActors::Debug::GetEntityFromActor(*DebugActor, AgentComp);	
			SetCachedEntity(EntityHandle, EntityManager);
		}
		CachedDebugActor = DebugActor;
	}
	else if (CachedDebugActor)
	{
		ClearCachedEntity();
		CachedDebugActor = nullptr;
	}
	else if (CachedEntity.IsValid() == true && EntityManager.IsEntityValid(CachedEntity) == false)
	{
		ClearCachedEntity();
	}

	if (CachedEntity.IsValid() && EntityManager.IsEntityValid(CachedEntity))
	{
		FMassEntityView EntityView(EntityManager, CachedEntity);

		FInstancedActorsFragment* InstancedActorFragment = EntityView.GetFragmentDataPtr<FInstancedActorsFragment>();
		if (InstancedActorFragment)
		{
			AddTextLine(FString::Printf(TEXT("{Green}IA Entity: {White}%s"), *CachedEntity.DebugGetDescription()));
			if (InstancedActorFragment->InstanceData.IsValid())
			{
				UInstancedActorsData& InstancedActorData = *InstancedActorFragment->InstanceData;

				AddTextLine(FString::Printf(TEXT("{Green}IAM: {White}%s"), *InstancedActorData.GetManagerChecked().GetName()));
				AddTextLine(FString::Printf(TEXT("{Green}IAD: {White}%s"), *InstancedActorData.GetDebugName(/*bCompact*/true)));
				
				if (const FInstancedActorsSettings* Settings = InstancedActorData.GetSettingsPtr<const FInstancedActorsSettings>())
				{
					AddTextLine(FString::Printf(TEXT("{Green}Settings: {White}%s"), *Settings->DebugToString()));
				}
				else
				{
					AddTextLine(FString::Printf(TEXT("{Green}Settings: {Red}None")));
				}
				InstancedActorData.ForEachVisualization([&](uint8 VisualizationIndex, const FInstancedActorsVisualizationInfo& Visualization)
				{
					FString VisualizationString;
					FInstancedActorsVisualizationInfo::StaticStruct()->ExportText(VisualizationString, &Visualization, /*Defaults*/nullptr, /*OwnerObject*/&InstancedActorData, PPF_None, /*ExportRootScope*/nullptr);
					AddTextLine(FString::Printf(TEXT("{Green}Visualization %u: {White}%s"), VisualizationIndex, *VisualizationString));

					return true;
				});
			}
			else
			{
				AddTextLine(FString::Printf(TEXT("{Green}IAD: {Red}None")));
			}
			AddTextLine(FString::Printf(TEXT("{Green}Index: {White}%s"), *InstancedActorFragment->InstanceIndex.GetDebugName()));
		}
	}
}

#endif // FN_WITH_GAMEPLAY_DEBUGGER && WITH_INSTANCEDACTORS_DEBUG && WITH_MASSENTITY_DEBUG
