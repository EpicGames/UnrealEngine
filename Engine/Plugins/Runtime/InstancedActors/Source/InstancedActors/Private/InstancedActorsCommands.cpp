// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsCommands.h"
#include "InstancedActorsDebug.h"
#include "MassEntityTypes.h"
#include "MassLODFragments.h"
#include "MassDistanceLODProcessor.h"
#include "MassStationaryISMSwitcherProcessor.h"
#include "InstancedActorsVisualizationProcessor.h"
#include "InstancedActorsData.h"

#ifndef INSTANCEDACTORS_AS_SMARTOBJECTS
#define INSTANCEDACTORS_AS_SMARTOBJECTS 1
#endif // INSTANCEDACTORS_AS_SMARTOBJECTS

#if INSTANCEDACTORS_AS_SMARTOBJECTS
#include "MassSmartObjectRegistration.h"
#endif // INSTANCEDACTORS_AS_SMARTOBJECTS

#if CSV_PROFILER_STATS || WITH_INSTANCEDACTORS_DEBUG
#	define DEBUG_NAME(Name) , FName(TEXT(Name))
#else
#	define DEBUG_NAME(Name)
#endif // CSV_PROFILER_STATS || WITH_MASSENTITY_DEBUG

namespace UE::InstancedActors
{	
	FMassTagBitSet& GetDetailedLODTags()
	{
		static FMassTagBitSet DetailedLODTags = UE::Mass::Utils::ConstructTagBitSet<EMassCommandCheckTime::CompileTimeCheck
			, FMassDistanceLODProcessorTag				// UMassDistanceLODProcessor requirement
			, FMassCollectDistanceLODViewerInfoTag		// UMassLODDistanceCollectorProcessor requirement
			, FMassStationaryISMSwitcherProcessorTag	// UMassStationaryISMSwitcherProcessor requirement
			, FInstancedActorsVisualizationProcessorTag	// UInstancedActorsVisualizationProcessor requirement
#if INSTANCEDACTORS_AS_SMARTOBJECTS
			, FMassInActiveSmartObjectsRangeTag
#endif // INSTANCEDACTORS_AS_SMARTOBJECTS
		>();

		return DetailedLODTags;
	}

	void FBulkLODTagsChangeCommandBase::Add(TNotNull<UInstancedActorsData*> InstanceData, EInstancedActorsBulkLOD NewLOD)
	{
		Super::Add(InstanceData->Entities);
		IADataToNotify.Emplace(InstanceData, NewLOD);
	}

	void FBulkLODTagsChangeCommandBase::Run(FMassEntityManager& EntityManager)
	{
		Super::Run(EntityManager);

		for (auto Pair : IADataToNotify)
		{
			UInstancedActorsData* IAData = Pair.Get<0>().Get();
			const EInstancedActorsBulkLOD LOD = Pair.Get<1>();

			// notify the data that it's LOD has changed
			IAData->OnBulkLODChanged(LOD);
		}
	}

	void FBulkLODTagsChangeCommandBase::Reset()
	{
		Super::Reset();
		IADataToNotify.Reset();
	}

	FEnableDetailedLODCommand::FEnableDetailedLODCommand()
		: Super(EMassCommandOperationType::Add
			, GetDetailedLODTags()
			, FMassTagBitSet{}
			DEBUG_NAME("DetailedLODEnable"))
	{
	}

	FEnableBatchLODCommand::FEnableBatchLODCommand()
		: Super(EMassCommandOperationType::Remove
			, FMassTagBitSet{}
			, GetDetailedLODTags()
			DEBUG_NAME("BatchLODEnable"))
	{
	}
} // UE::InstancedActors

#undef DEBUG_NAME
