// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsComponent.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"
#include "ServerInstancedActorsSpawnerSubsystem.h"
#include "InstancedActorsSettings.h"
#include "Net/UnrealNetwork.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsComponent)


UInstancedActorsComponent::UInstancedActorsComponent()
{
	bWantsInitializeComponent = true;
}

void UInstancedActorsComponent::OnServerPreSpawnInitForInstance(FInstancedActorsInstanceHandle InInstanceHandle)
{
	InstanceHandle = InInstanceHandle;
}

void UInstancedActorsComponent::InitializeComponentForInstance(FInstancedActorsInstanceHandle InInstanceHandle)
{
	InstanceHandle = InInstanceHandle;
}

void UInstancedActorsComponent::GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);

	DOREPLIFETIME_CONDITION_NOTIFY(UInstancedActorsComponent, InstanceHandle, COND_InitialOnly, REPNOTIFY_OnChanged);
}

void UInstancedActorsComponent::OnRep_InstanceHandle()
{
	// Note: The client may not have loaded InstanceHandle.InstancedActorData yet, resulting in an invalid InstanceHandle. Once the client
	// 		 completes the load however, we'll get another OnRep_InstanceHandle with the fixed up InstancedActorData pointer.
	if (InstanceHandle.IsValid())
	{
		check(GetOwner());
		InstanceHandle.GetInstanceActorDataChecked().SetReplicatedActor(InstanceHandle.GetInstanceIndex(), *GetOwner());
	}
}

void UInstancedActorsComponent::InitializeComponent()
{
	Super::InitializeComponent();

	// @todo Add support for non-replay NM_Standalone where we should call OnInstancedActorComponentInitialize
	UWorld* World = GetWorld();
	if (World && World->GetNetMode() != NM_Client)
	{
		UServerInstancedActorsSpawnerSubsystem* ServerInstancedActorSpawnerSubystem = UE::InstancedActors::Utils::GetServerInstancedActorsSpawnerSubsystem(*World);
		if (IsValid(ServerInstancedActorSpawnerSubystem))
		{
			ServerInstancedActorSpawnerSubystem->OnInstancedActorComponentInitialize(*this);
		}
	}
}

void UInstancedActorsComponent::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	// Deregister Actor from entity on clients
	if (GetNetMode() == NM_Client && InstanceHandle.IsValid())
	{
		AActor* Owner = GetOwner();
		if (ensure(Owner))
		{
			InstanceHandle.GetInstanceActorDataChecked().ClearReplicatedActor(InstanceHandle.GetInstanceIndex(), *Owner);
		}
	}

	// @todo Callback from UMassActorSpawnerSubsystem when if/when we're released
	InstanceHandle.Reset();

	Super::EndPlay(EndPlayReason);
}

FMassEntityHandle UInstancedActorsComponent::GetMassEntityHandle() const
{
	if (InstanceHandle.IsValid())
	{
		return InstanceHandle.GetInstanceActorDataChecked().GetEntity(InstanceHandle.GetInstanceIndex());
	}

	return FMassEntityHandle();
}

TSharedPtr<FMassEntityManager> UInstancedActorsComponent::GetMassEntityManager() const
{
	if (InstanceHandle.IsValid())
	{
		return InstanceHandle.GetManagerChecked().GetMassEntityManager();
	}

	return TSharedPtr<FMassEntityManager>();
}

FMassEntityManager& UInstancedActorsComponent::GetMassEntityManagerChecked() const
{
	return InstanceHandle.GetManagerChecked().GetMassEntityManagerChecked();
}
