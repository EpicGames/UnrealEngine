// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsCustomVersion.h"

#include "Serialization/CustomVersion.h"

const FGuid FInstancedActorsCustomVersion::GUID(0xA6EBC74A, 0x3BAD46B8, 0xAA30D6B2, 0x7D58EA25);

// Register the custom version with core
FCustomVersionRegistration GRegisterInstancedActorsCustomVersion(FInstancedActorsCustomVersion::GUID, FInstancedActorsCustomVersion::LatestVersion, TEXT("InstancedActors"));
