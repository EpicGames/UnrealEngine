// Copyright Epic Games, Inc. All Rights Reserved.


#include "InstancedActorsDebug.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsModifierVolumeComponent.h"
#include "InstancedActorsModifiers.h"
#include "InstancedActorsTypes.h"

#include "Containers/UnrealString.h"
#include "EngineUtils.h"
#include "Engine/Engine.h"
#include "HAL/FileManager.h"
#include "HAL/IConsoleManager.h"
#include "Serialization/Formatters/JsonArchiveOutputFormatter.h"
#include "Serialization/MemoryWriter.h"


#if WITH_INSTANCEDACTORS_DEBUG

namespace UE::InstancedActors::Debug
{
namespace CVars
{
	int32 DebugManagerLoading = 0;
	FAutoConsoleVariableRef CVarDebugManagerLoading(
		TEXT("IA.DebugManagerLoading"),
		DebugManagerLoading,
		TEXT("Debug draw Instanced Actor Managers as they load\n")
		TEXT("0 = Disabled, 1 = DebugDraw, 2 = VisLog, 3 = Both"),
		ECVF_Cheat);

	int32 DebugInstanceLoading = 0;
	FAutoConsoleVariableRef CVarDebugInstances(
		TEXT("IA.DebugInstanceLoading"),
		DebugInstanceLoading,
		TEXT("Debug draw Instanced Actor instances & add on screen debug messages about instance counts, as they load\n")
		TEXT("0 = Disabled, 1 = DebugDraw, 2 = VisLog, 3 = Both"),
		ECVF_Cheat);

	int32 DebugModifiers = 0;
	FAutoConsoleVariableRef CVarDebugModifiers(
		TEXT("IA.DebugModifiers"),
		DebugModifiers,
		TEXT("Debug draw Instanced Actor Modifier Volumes / instance modifications\n")
		TEXT("0 = Disabled, 1 = DebugDraw, 2 = VisLog, 3 = Both"),
		ECVF_Cheat);
}

namespace Cmds
{
	void AuditInstances(const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar);
	void AuditPersistence(const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar);

	static FAutoConsoleCommand AuditInstancesCommand(
		TEXT("IA.AuditInstances"),
		TEXT("Generates instance manager stats. Pass true as first arg to additionally debug draw instance info. ")
		TEXT("A debug draw duration in seconds can optionally be passed as the second arg (pass -1 for permanent draws)"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(AuditInstances)
	);

#if WITH_TEXT_ARCHIVE_SUPPORT
	static FAutoConsoleCommand AuditPersistenceCommand(
		TEXT("IA.AuditPersistence"),
		TEXT("Performs a dummy persistence serialization for all IAM's and dumps the output"),
		FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(AuditPersistence)
	);
#endif // WITH_TEXT_ARCHIVE_SUPPORT

	void AuditInstances(const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
	{
		bool bDebugDraw = Args.Num() ? FCString::ToBool(*Args[0]) : false;
		float DebugDrawDuration = Args.Num() > 1 ? FCString::Atof(*Args[1]) : 10.0f;

		int32 InstanceCountGrandTotal = 0;

		for (TActorIterator<AInstancedActorsManager> MangerIt(World); MangerIt; ++MangerIt)
		{
			AInstancedActorsManager* Manager = *MangerIt;
			check(Manager);

			Manager->AuditInstances(Ar, bDebugDraw, DebugDrawDuration);

			InstanceCountGrandTotal += Manager->GetNumValidInstances();
		}

		Ar.Logf(TEXT("Total Num Instances: %d"), InstanceCountGrandTotal);
	}

#if WITH_TEXT_ARCHIVE_SUPPORT
	void AuditPersistence(const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
	{
		if (!ensureMsgf(World->IsGameWorld(), TEXT("IA.AuditPersistence called for non-game world")))
		{
			return;
		}

		FString IAMPersistenceAuditFilename = FPaths::CreateTempFilename(*FPaths::ProjectLogDir(), TEXT("IAMPersistenceAudit"), TEXT(".json"));
		TUniquePtr<FArchive> IAMPersistenceAuditFileAr(IFileManager::Get().CreateFileWriter(*IAMPersistenceAuditFilename));
		if (!ensureMsgf(IAMPersistenceAuditFileAr.IsValid(), TEXT("Couldn't create IAM Persistence Audit File at %s"), *IAMPersistenceAuditFilename))
		{
			return;
		}
		IAMPersistenceAuditFileAr->ArIsSaveGame = true;
		FJsonArchiveOutputFormatter JsonFormatter(*IAMPersistenceAuditFileAr);
		FStructuredArchive StructuredArchive(JsonFormatter);
		FStructuredArchiveSlot RootSlot = StructuredArchive.Open();
		FStructuredArchiveStream Stream = RootSlot.EnterStream();

		TArray<uint8> ScratchMem;

		for (TActorIterator<AInstancedActorsManager> MangerIt(World); MangerIt; ++MangerIt)
		{
			AInstancedActorsManager* Manager = *MangerIt;
			check(Manager);

			FStructuredArchiveSlot StreamElement = Stream.EnterElement();
			FStructuredArchiveRecord Record = StreamElement.EnterRecord();

#if WITH_EDITOR
			FString Label = Manager->GetActorLabel();
			Record << SA_VALUE(TEXT("Label"), Label);
#endif
			FString Name = Manager->GetPathName();
			Record << SA_VALUE(TEXT("Name"), Name);
		
		
			ScratchMem.Reset();
			FMemoryWriter MemWriter(ScratchMem);
			MemWriter.ArIsSaveGame = true;
			Manager->Serialize(MemWriter);
			int32 BinarySize = ScratchMem.Num();
			Record << SA_VALUE(TEXT("BinarySize"), BinarySize);

			Manager->Serialize(Record);
		}

		Ar.Logf(ELogVerbosity::Display, TEXT("IAM Persistence Audit saved to:\n\t%s"), *IAMPersistenceAuditFilename);
	}
#endif // WITH_TEXT_ARCHIVE_SUPPORT
}

void DebugDrawManager(const int32& DebugDrawMode, const AInstancedActorsManager& Manager)
{
	const FBox ManagerBounds = Manager.GetInstanceBounds();
	UE::InstancedActors::Debug::DrawDebugSolidBox(DebugDrawMode, Manager.GetWorld(), /*LogOwner*/&Manager, "LogInstancedActors", ELogVerbosity::Display, ManagerBounds, FColor::Purple, TEXT("%s"), *Manager.GetName());

#if ENABLE_VISUAL_LOG
	if (UE::InstancedActors::Debug::ShouldVisLog(DebugDrawMode))
	{
		FStringOutputDevice InstanceAudit;
		Manager.AuditInstances(InstanceAudit);
		UE_VLOG(&Manager, LogInstancedActors, Display, TEXT("%s"), *InstanceAudit);
	}
#endif
}

void LogInstanceCountsOnScreen(const int32& DebugDrawMode, const AInstancedActorsManager& Manager, float TimeToDisplay, FColor Color)
{
	if (ShouldDebugDraw(DebugDrawMode))
	{
		GEngine->AddOnScreenDebugMessage(INDEX_NONE, TimeToDisplay, Color, FString::Printf(TEXT("%s:"), *Manager.GetPathName()));
		for (UInstancedActorsData* InstanceData : Manager.GetAllInstanceData())
		{
			check(IsValid(InstanceData));
			GEngine->AddOnScreenDebugMessage(INDEX_NONE, TimeToDisplay, Color, FString::Printf(TEXT("\t%d %s"), InstanceData->GetNumInstances(), *InstanceData->ActorClass->GetName()));
		}
	}
}

void DebugDrawAllInstanceLocations(const int32& DebugDrawMode, ELogVerbosity::Type Verbosity, const AInstancedActorsManager& Manager, const TOptional<FColor> Color, const UObject* LogOwner, const FName& CategoryName)
{
	if (DebugDrawMode <= UE::InstancedActors::Debug::DrawMode::None)
	{
		return;
	}

	auto MakeRandomColorForInstanceActorClass = [](const FInstancedActorsInstanceHandle& InstanceHandle)
		{
			return FColor::MakeRandomSeededColor(GetTypeHash(InstanceHandle.GetInstanceActorDataChecked().ActorClass->GetFName()));
		};

	Manager.ForEachInstance([&](const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext)
		{
			const FColor InstanceColor = Color.IsSet() ? *Color : MakeRandomColorForInstanceActorClass(InstanceHandle);
			UE::InstancedActors::Debug::DebugDrawLocation(DebugDrawMode, InstanceHandle.GetManager()->GetWorld(), /*LogOwner*/LogOwner, CategoryName, Verbosity, InstanceTransform.GetLocation(), /*Size*/30.0f, InstanceColor, TEXT("%d"), InstanceHandle.GetIndex());

			return true;
		});
}

void DebugDrawModifierVolumeBounds(const int32& DebugDrawMode, const UInstancedActorsModifierVolumeComponent& ModifierVolume, const FColor& Color)
{
	switch (ModifierVolume.Shape)
	{
	case EInstancedActorsVolumeShape::Box:
	{
		const FBox BoxBounds = ModifierVolume.Bounds.GetBox();
		UE::InstancedActors::Debug::DrawDebugSolidBox(DebugDrawMode, ModifierVolume.GetWorld(), /*LogOwner*/ModifierVolume.GetWorld(), "LogInstancedActors", ELogVerbosity::Display, BoxBounds, Color, TEXT(""));
		break;
	}
	case EInstancedActorsVolumeShape::Sphere:
	{
		const FSphere SphereBounds = ModifierVolume.Bounds.GetSphere();
		UE::InstancedActors::Debug::DebugDrawSphere(DebugDrawMode, ModifierVolume.GetWorld(), /*LogOwner*/ModifierVolume.GetWorld(), "LogInstancedActors", ELogVerbosity::Display, SphereBounds, Color, TEXT(""));
		break;
	}
	default:
		checkNoEntry();
	}
}

void DebugDrawModifierVolumeAddedToManager(const int32& DebugDrawMode, const AInstancedActorsManager& Manager, const UInstancedActorsModifierVolumeComponent& AddedModifierVolume)
{
	if (ShouldDebugDraw(DebugDrawMode))
	{
#if ENABLE_DRAW_DEBUG
		DrawDebugDirectionalArrow(Manager.GetWorld(), AddedModifierVolume.GetComponentLocation(), Manager.GetActorLocation(), /*ArrowSize*/30.0f, FColorList::LightBlue, /*bPersistentLines*/true);
#endif // ENABLE_DRAW_DEBUG
	}
	if (ShouldVisLog(DebugDrawMode))
	{
#if ENABLE_VISUAL_LOG
		UE_VLOG_ARROW(AddedModifierVolume.GetWorld(), "LogInstancedActors", Display, AddedModifierVolume.GetComponentLocation(), Manager.GetActorLocation(), FColorList::LightBlue, TEXT("%s"), *Manager.GetName());
#endif // ENABLE_VISUAL_LOG
	}
}
} // namespace UE::InstancedActors::Debug

#endif // WITH_INSTANCEDACTORS_DEBUG
