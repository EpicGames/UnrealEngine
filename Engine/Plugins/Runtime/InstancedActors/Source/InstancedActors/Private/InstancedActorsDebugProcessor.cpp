// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsDebugProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsDebugProcessor)

#if WITH_INSTANCEDACTORS_DEBUG

#include "MassEntitySubsystem.h"
#include "MassEntityView.h"
#include "Math/ColorList.h"
#include "VisualLogger/VisualLogger.h"
#include "HAL/IConsoleManager.h"
#include "MassDebugger.h"

#include "MassDebuggerSubsystem.h"
#include "MassCommonFragments.h"
#include "MassActorSubsystem.h"
#include "MassRepresentationFragments.h"
#include "MassDistanceLODProcessor.h"
#include "MassRepresentationSubsystem.h"
#include "InstancedActorsTypes.h"


UE_DISABLE_OPTIMIZATION_SHIP

namespace UE::InstancedActors::Debug
{
	bool bDebugDrawMissingActors = false;
	bool bDebugDrawDetailedCurrentRepresentation = false;
	bool bDebugDrawPrevRepresentation = true;
	bool bDebugDrawAllEntities = false;

	bool ShouldDebugDraw()
	{
		return bDebugDrawMissingActors || bDebugDrawDetailedCurrentRepresentation || bDebugDrawAllEntities;
	}

	namespace
	{
		FAutoConsoleVariableRef AnonymousCVars[] = {
			{TEXT("IA.debug.MissingActors"), bDebugDrawMissingActors
				, TEXT("When enabled will debug-draw information related to actor representation "
				"of instanced actors. Green is good, Red means the actor is not none but invalid, "
				"Blue indicated a valid spawn request present and "
				"Magenta indicates a no-expected-actor state."), ECVF_Cheat},
			{TEXT("IA.debug.CurrentRepresentation"), bDebugDrawDetailedCurrentRepresentation
				, TEXT("When enabled will debug draw data related to current representation of instanced "
				"actors at Details batch LOD level. Green indicates HighResSpawnedActor, DarkOliveGreen "
				"indicates LowResSpawnedActor, Magenta indicates StaticMeshInstance and Red means Off."), ECVF_Cheat},
			{TEXT("IA.debug.PreviousRepresentation"), bDebugDrawPrevRepresentation
				, TEXT("When enabled (the default value) will debug draw data related to previous "
				"representation of instanced actors being drawn. Green indicates HighResSpawnedActor, DarkOliveGreen "
				"indicates LowResSpawnedActor, Magenta indicates StaticMeshInstance, Purple indicates SkinnedMeshInstance and Red means Off."
				"Note that the data will be drawn only if Previous Representation differs from the Current Representation"), ECVF_Cheat},
			{TEXT("IA.debug.CurrentRepresentationAll"), bDebugDrawAllEntities
				, TEXT("When enabled will debug draw data related to current representation of all instanced "
				"actors.\nFor instanced at Detailed batch LOD level Green indicates HighResSpawnedActor, DarkOliveGreen "
				"indicates LowResSpawnedActor, Magenta indicates StaticMeshInstance, Purple indicates SkinnedMeshInstance and Red means Off."
				"\nFor instanced not at Detailed level: Yellow indicates HighResSpawnedActor, Orange indicates "
				"LowResSpawnedActor, NeonPink indicates StaticMeshInstance, CadetBlue indicates SkinnedMeshInstance, and Red means Off."), ECVF_Cheat},
	};
	}

	FColor GetDetailedColor(const EMassRepresentationType Representation)
	{
		// Red means Off
		FColor DrawColor = FColor::Red;
		switch (Representation)
		{
		case EMassRepresentationType::HighResSpawnedActor:
			DrawColor = FColor::Green;
			break;
		case EMassRepresentationType::LowResSpawnedActor:
			DrawColor = FColorList::DarkOliveGreen;
			break;
		case EMassRepresentationType::SkinnedMeshInstance:
			DrawColor = FColor::Purple;
			break;
		case EMassRepresentationType::StaticMeshInstance:
			DrawColor = FColor::Magenta;
			break;
		}
		return DrawColor;
	}

	FColor GetBatchColor(const EMassRepresentationType Representation)
	{
		// Red means Off
		FColor DrawColor = FColor::Red;
		switch (Representation)
		{
		case EMassRepresentationType::HighResSpawnedActor:
			DrawColor = FColorList::Yellow;
			break;
		case EMassRepresentationType::LowResSpawnedActor:
			DrawColor = FColorList::Orange;
			break;
		case EMassRepresentationType::SkinnedMeshInstance:
			DrawColor = FColorList::CadetBlue;
			break;
		case EMassRepresentationType::StaticMeshInstance:
			DrawColor = FColorList::NeonPink;
			break;
		}	
		return DrawColor;
	}

	// this color indicates that the entity is not being "represented", meaning it doesn't have a FMassActorFragment
	const FColor NotRepresentedColor = FColorList::SpringGreen;

	FAutoConsoleCommandWithWorldAndArgs ResetSpawnRequests(
		TEXT("IA.debug.ResetSpawnRequests")
		, TEXT("Aborts all actor spawn requests issued for instanced actors at Detailed Batch LOD level. "
			"This operation will result in actor spawning being re-requested.")
		, FConsoleCommandWithWorldAndArgsDelegate::CreateLambda([](const TArray<FString>& InParams, UWorld* InWorld)
			{
				if (!InWorld)
				{
					return;
				}

				if (UMassEntitySubsystem* EntitySubsystem = UWorld::GetSubsystem<UMassEntitySubsystem>(InWorld))
				{
					FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();

					FMassEntityQuery DetailedLODEntityQuery(EntityManager.AsShared());
					DetailedLODEntityQuery.AddRequirement<FInstancedActorsFragment>(EMassFragmentAccess::None);
					DetailedLODEntityQuery.AddRequirement<FMassActorFragment>(EMassFragmentAccess::ReadOnly);
					DetailedLODEntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
					DetailedLODEntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadOnly);
				
					FMassExecutionContext Context = EntityManager.CreateExecutionContext(0);
					DetailedLODEntityQuery.ForEachEntityChunk(Context, [World = InWorld](FMassExecutionContext& Context)
						{
							const TConstArrayView<FMassActorFragment> ActorFragmentsList = Context.GetFragmentView<FMassActorFragment>();
							const TArrayView<FMassRepresentationFragment> RepresentationsList = Context.GetMutableFragmentView<FMassRepresentationFragment>();
							const UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;

							if (ensure(RepresentationSubsystem))
							{
								for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
								{
									if (ActorFragmentsList[EntityIt].IsValid())
									{
										continue;
									}

									FMassRepresentationFragment& Representation = RepresentationsList[EntityIt];

									if (Representation.ActorSpawnRequestHandle.IsValid()
										&& ensure(RepresentationSubsystem->GetActorSpawnerSubsystem()))
									{
										RepresentationSubsystem->GetActorSpawnerSubsystem()->RemoveActorSpawnRequest(Representation.ActorSpawnRequestHandle);
									}
								}
							}
						});
				}
			})
	);
}

//-----------------------------------------------------------------------------
// UInstancedActorsDebugProcessor
//-----------------------------------------------------------------------------
UInstancedActorsDebugProcessor::UInstancedActorsDebugProcessor()
	: DetailedLODEntityQuery(*this)
	, DebugAllEntityQuery(*this)
{
	bAutoRegisterWithProcessingPhases = true;

	// running in StartPhysics to ensure this processor runs after everything in EMassProcessingPhase::PrePhysics (where most of the Mass-run logic happens)
	ProcessingPhase = EMassProcessingPhase::StartPhysics;
	ExecutionFlags = (int32)(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);

	bRequiresGameThreadExecution = true; // for debug drawing
}

void UInstancedActorsDebugProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	DebugAllEntityQuery.AddRequirement<FInstancedActorsFragment>(EMassFragmentAccess::None);
	DebugAllEntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	DebugAllEntityQuery.AddRequirement<FMassActorFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	DebugAllEntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	DebugAllEntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	DebugAllEntityQuery.AddSubsystemRequirement<UMassRepresentationSubsystem>(EMassFragmentAccess::ReadOnly);

	DetailedLODEntityQuery = DebugAllEntityQuery;
	DetailedLODEntityQuery.AddTagRequirement<FMassDistanceLODProcessorTag>(EMassFragmentPresence::All);
	DetailedLODEntityQuery.AddSubsystemRequirement<UMassRepresentationSubsystem>(EMassFragmentAccess::ReadOnly);

	ProcessorRequirements.AddSubsystemRequirement<UMassDebuggerSubsystem>(EMassFragmentAccess::ReadOnly);
	ProcessorRequirements.AddSubsystemRequirement<UMassLODSubsystem>(EMassFragmentAccess::ReadOnly);
}

void UInstancedActorsDebugProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	constexpr double BoxSize = 30.;

	if (UE::InstancedActors::Debug::ShouldDebugDraw() == false)
	{
		return;
	}

	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}

	const UMassDebuggerSubsystem* Debugger = Context.GetSubsystem<UMassDebuggerSubsystem>();
	if (Debugger == nullptr)
	{
		return;
	}

	const UMassLODSubsystem& LODSubsystem = Context.GetSubsystemChecked<UMassLODSubsystem>();
	// note we're copying the array on purpose since we intend to use parallel-for here (eventually)
	TArray<FViewerInfo> Viewers = LODSubsystem.GetViewers();

	// we don't care about streaming-sources
	for (int32 ViewerIndex = Viewers.Num() - 1; ViewerIndex >= 0; --ViewerIndex)
	{
		if (Viewers[ViewerIndex].StreamingSourceName.IsNone() == false)
		{
			Viewers.RemoveAtSwap(ViewerIndex, EAllowShrinking::No);
		}
	}

	const FVector ReferenceLocation = Viewers.Num() ? Viewers[0].Location : FVector(0);
	
	auto ExecFunction = [this, Debugger, World = World, ReferenceLocation](FMassExecutionContext& Context)
		{
			const TConstArrayView<FTransformFragment> TransformsList = Context.GetFragmentView<FTransformFragment>();
			const TConstArrayView<FMassActorFragment> ActorFragmentsList = Context.GetFragmentView<FMassActorFragment>();
			const TConstArrayView<FMassRepresentationFragment> RepresentationsList = Context.GetFragmentView<FMassRepresentationFragment>();
			const UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
			const bool bFullyRepresented = !ActorFragmentsList.IsEmpty() && !RepresentationsList.IsEmpty() && RepresentationSubsystem;
			const bool bDetailedLOD = (UE::InstancedActors::Debug::bDebugDrawAllEntities == false) || Context.DoesArchetypeHaveTag<FMassDistanceLODProcessorTag>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
				const FTransformFragment& Transform = TransformsList[EntityIt];
				const FMassRepresentationFragment& Representation = RepresentationsList[EntityIt];

				FVector Offset(0, 0, 150);
				if (bFullyRepresented)
				{
					if (UE::InstancedActors::Debug::bDebugDrawMissingActors)
					{
						if (const AActor* RepresentationActor = ActorFragmentsList[EntityIt].Get())
						{
							DrawDebugSolidBox(World, RepresentationActor->GetActorLocation() + Offset, FVector(BoxSize), IsValid(RepresentationActor) ? FColor::Green : FColor::Red);
						}
						else
						{
							Offset.Z += BoxSize * 2;
							FColor DrawColor = FColor::Red;
							if (Representation.ActorSpawnRequestHandle.IsValid() == false)
							{
								Offset.Z += BoxSize * 2;
								DrawColor = FColor::Magenta;
							}
							else if (ensure(RepresentationSubsystem) && ensure(RepresentationSubsystem->GetActorSpawnerSubsystem())
								&& RepresentationSubsystem->GetActorSpawnerSubsystem()->IsSpawnRequestHandleValid(Representation.ActorSpawnRequestHandle))
							{
								Offset.Z += BoxSize * 4;
								DrawColor = FColor::Blue;
							}

							DrawDebugSolidBox(World, Transform.GetTransform().GetLocation() + Offset, FVector(BoxSize), DrawColor);
						}
					}
					if (UE::InstancedActors::Debug::bDebugDrawDetailedCurrentRepresentation || UE::InstancedActors::Debug::bDebugDrawAllEntities)
					{
						DrawDebugBox(World, Transform.GetTransform().GetLocation() + Offset, FVector(BoxSize * 1.5)
							, bDetailedLOD 
								? UE::InstancedActors::Debug::GetDetailedColor(Representation.CurrentRepresentation) 
								: UE::InstancedActors::Debug::GetBatchColor(Representation.CurrentRepresentation));
					}
					if (UE::InstancedActors::Debug::bDebugDrawPrevRepresentation && (Representation.PrevRepresentation != Representation.CurrentRepresentation))
					{
						DrawDebugBox(World, Transform.GetTransform().GetLocation() + Offset, FVector(BoxSize * 2.)
							, bDetailedLOD 
								? UE::InstancedActors::Debug::GetDetailedColor(Representation.PrevRepresentation)
								: UE::InstancedActors::Debug::GetBatchColor(Representation.PrevRepresentation));
					}
				}
				else
				{
					// only a Transform is available
					DrawDebugBox(World, Transform.GetTransform().GetLocation() + Offset, FVector(BoxSize * 1.5), UE::InstancedActors::Debug::NotRepresentedColor);
				}
			}
		};
	
	if (UE::InstancedActors::Debug::bDebugDrawAllEntities)
	{
		DebugAllEntityQuery.ForEachEntityChunk(Context, ExecFunction);
	}
	else
	{
		DetailedLODEntityQuery.ForEachEntityChunk(Context, ExecFunction);
	}
}
UE_ENABLE_OPTIMIZATION_SHIP

#else // WITH_INSTANCEDACTORS_DEBUG

UInstancedActorsDebugProcessor::UInstancedActorsDebugProcessor()
{
	bAutoRegisterWithProcessingPhases = false;
}

void UInstancedActorsDebugProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
}

void UInstancedActorsDebugProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
}

#endif // WITH_INSTANCEDACTORS_DEBUG
