// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsIndex.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsIndex)


FInstancedActorsInstanceHandle::FInstancedActorsInstanceHandle(UInstancedActorsData& InInstancedActorData, FInstancedActorsInstanceIndex InIndex)
	: InstancedActorData(&InInstancedActorData)
	, Index(InIndex)
{
}

FString FInstancedActorsInstanceIndex::GetDebugName() const
{
	return FString::Printf(TEXT("%d"), Index);
}

bool FInstancedActorsInstanceHandle::IsValid() const 
{
	return InstancedActorData.IsValid() && Index.IsValid();
}

UInstancedActorsData& FInstancedActorsInstanceHandle::GetInstanceActorDataChecked() const 
{ 
	UInstancedActorsData* RawInstancedActorData = InstancedActorData.Get();
	check(RawInstancedActorData);
	return *InstancedActorData; 
}

AInstancedActorsManager* FInstancedActorsInstanceHandle::GetManager() const
{
	if (UInstancedActorsData* RawInstancedActorData = InstancedActorData.Get())
	{
		return RawInstancedActorData->GetManager();
	}

	return nullptr;
}

AInstancedActorsManager& FInstancedActorsInstanceHandle::GetManagerChecked() const
{
	return GetInstanceActorDataChecked().GetManagerChecked();
}

FString FInstancedActorsInstanceHandle::GetDebugName() const
{
	UInstancedActorsData* RawInstancedActorData = InstancedActorData.Get();
	return FString::Printf(TEXT("%s : %s"), RawInstancedActorData ? *RawInstancedActorData->GetDebugName() : TEXT("null"), *Index.GetDebugName());
}

uint32 GetTypeHash(const FInstancedActorsInstanceHandle& Handle)
{
	uint32 Hash = 0;
	if (UInstancedActorsData* RawInstancedActorData = Handle.InstancedActorData.Get())
	{
		Hash = GetTypeHash(RawInstancedActorData);
	}
	Hash = HashCombine(Hash, Handle.GetIndex());

	return Hash;
}
