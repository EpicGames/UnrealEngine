// Copyright Epic Games, Inc. All Rights Reserved.


#include "InstancedActorsInitializerProcessor.h"

#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"

#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassEntityQuery.h"
#include "MassActorSubsystem.h"
#include "MassSignalSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsInitializerProcessor)


UInstancedActorsInitializerProcessor::UInstancedActorsInitializerProcessor()
	: EntityQuery(*this)
{
	bAutoRegisterWithProcessingPhases = false;
}

void UInstancedActorsInitializerProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	ProcessorRequirements.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);

	EntityQuery.AddRequirement<FMassActorInstanceFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FInstancedActorsFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassGuidFragment>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
}

/**
* Initializes transform and GUID fragments. For the transform, we're going to apply the InstanceActor's parent transform (i.e.,
* the ActorInstanceManager transform). As for the GUID, we'll assign it to an incremental index, that grows each time a fragment
* is initialized.
*/
template<bool bApplyManagerTranslationOnly, bool bFilterInstanceTransforms>
void InitInstanceFragments(UInstancedActorsData* InstanceData, int32& NextInstanceIndex, const FVector& ManagerLocation
	, const FTransform& ManagerTransform, FMassExecutionContext& Context, TArray<FMassEntityHandle>& InOutEntitiesToSignal)
{
	const int32 NumEntities = Context.GetNumEntities();
	InOutEntitiesToSignal.Reserve(InOutEntitiesToSignal.Num() + NumEntities);

	TArrayView<FMassActorInstanceFragment> MassActorInstanceFragments = Context.GetMutableFragmentView<FMassActorInstanceFragment>();
	TArrayView<FInstancedActorsFragment> InstancedActorFragments = Context.GetMutableFragmentView<FInstancedActorsFragment>();
	TArrayView<FTransformFragment> TransformFragments = Context.GetMutableFragmentView<FTransformFragment>();
	TArrayView<FMassGuidFragment> GuidFragments = Context.GetMutableFragmentView<FMassGuidFragment>();

	// Block copy instance transforms if we don't need to filter for invalidated transforms
	if constexpr (!bFilterInstanceTransforms)
	{
		check(NextInstanceIndex + NumEntities <= InstanceData->InstanceTransforms.Num());
		check(TransformFragments.GetTypeSize() == InstanceData->InstanceTransforms.GetTypeSize());
		FMemory::Memcpy(TransformFragments.GetData(), &InstanceData->InstanceTransforms[NextInstanceIndex], NumEntities * TransformFragments.GetTypeSize());
	}

	AInstancedActorsManager& Manager = InstanceData->GetManagerChecked();
	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		FMassActorInstanceFragment& MassActorInstanceFragment = MassActorInstanceFragments[EntityIt];
		FInstancedActorsFragment& InstancedActorFragment = InstancedActorFragments[EntityIt];
		FTransformFragment& TransformFragment = TransformFragments[EntityIt];
		const FMassEntityHandle EntityHandle(Context.GetEntity(EntityIt));
		InOutEntitiesToSignal.Add(EntityHandle);

		if constexpr (bFilterInstanceTransforms)
		{
			// Skip invalidated (scale 0) instance transforms
			while (InstanceData->InstanceTransforms[NextInstanceIndex].GetScale3D().IsZero())
			{
				++NextInstanceIndex;
				check(InstanceData->InstanceTransforms.IsValidIndex(NextInstanceIndex));
			}

			// Copy local space transform
			TransformFragment.GetMutableTransform() = InstanceData->InstanceTransforms[NextInstanceIndex];
		}

		InstancedActorFragment.InstanceIndex = FInstancedActorsInstanceIndex(NextInstanceIndex);
		InstanceData->Entities[NextInstanceIndex] = EntityHandle;

		const int32 InstanceDataId = Manager.GetAllInstanceData().Find(InstanceData);
		check(InstanceDataId != INDEX_NONE);
		MassActorInstanceFragment.Handle = FActorInstanceHandle::MakeDehydratedActorHandle(Manager
			, FInstancedActorsInstanceIndex::BuildCompositeIndex(InstanceDataId, NextInstanceIndex));

		// @todo make GuidFragments required if we decide to go with deterministic entity naming/guid-ing
		if (GuidFragments.Num())
		{
			GuidFragments[EntityIt].Guid.D = NextInstanceIndex;
		}

		// Convert to world space
		if constexpr (bApplyManagerTranslationOnly)
		{
			TransformFragment.GetMutableTransform().AddToTranslation(ManagerLocation);
		}
		else
		{
			TransformFragment.GetMutableTransform() *= ManagerTransform;
		}

		++NextInstanceIndex;
	}
}

void UInstancedActorsInitializerProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	if (!ensureMsgf(Context.ValidateAuxDataType<FInstancedActorsMassSpawnData>(), TEXT("Execution context has invalid AuxData or it's not FInstancedActorsMassSpawnData. Entity transforms won't be initialized.")))
	{
		return;
	}

	FInstancedActorsMassSpawnData& AuxData = Context.GetMutableAuxData().GetMutable<FInstancedActorsMassSpawnData>();
	UInstancedActorsData* InstanceData = AuxData.InstanceData.Get();
	check(InstanceData);


	const AInstancedActorsManager& Manager = InstanceData->GetManagerChecked();
	const FVector ManagerLocation = Manager.GetActorLocation();
	const FTransform& ManagerTransform = Manager.GetActorTransform();
	const bool bApplyManagerTranslationOnly = (Manager.GetActorQuat().IsIdentity() && Manager.GetActorScale().Equals(FVector::OneVector));
	const bool bFilterInstanceTransforms = InstanceData->GetNumFreeInstances() > 0;
	
	int32 NumInitializedEntities = 0;
	int32 NextInstanceIndex = 0;

	TArray<FMassEntityHandle> EntitiesToSignal;

	EntityQuery.ForEachEntityChunk(Context, [InstanceData, &NumInitializedEntities, &NextInstanceIndex, &ManagerLocation, &ManagerTransform, bApplyManagerTranslationOnly, bFilterInstanceTransforms, &EntitiesToSignal](FMassExecutionContext& Context)
	{
		if (bApplyManagerTranslationOnly)
		{
			if (bFilterInstanceTransforms)
			{
				InitInstanceFragments<true, true>(InstanceData, NextInstanceIndex, ManagerLocation, ManagerTransform, Context, EntitiesToSignal);
			}
			else
			{
				InitInstanceFragments<true, false>(InstanceData, NextInstanceIndex, ManagerLocation, ManagerTransform, Context, EntitiesToSignal);
			}
		}
		else
		{
			if (bFilterInstanceTransforms)
			{
				InitInstanceFragments<false, true>(InstanceData, NextInstanceIndex, ManagerLocation, ManagerTransform, Context, EntitiesToSignal);
			}
			else
			{
				InitInstanceFragments<false, false>(InstanceData, NextInstanceIndex, ManagerLocation, ManagerTransform, Context, EntitiesToSignal);
			}
		}

		NumInitializedEntities += Context.GetNumEntities();
	});

	// Signal all entities inside the consolidated list
	if (EntitiesToSignal.Num())
	{
		UMassSignalSubsystem& SignalSubsystem = Context.GetMutableSubsystemChecked<UMassSignalSubsystem>();
		SignalSubsystem.SignalEntities(UE::Mass::Signals::ActorInstanceHandleChanged, EntitiesToSignal);
	}

#if DO_CHECK
	checkf(NumInitializedEntities == InstanceData->NumValidInstances
		, TEXT("UInstancedActorsInitializerProcessor expects to initialize all spawned entities at once and to have the same number of valid transforms to assign"));
#endif // DO_CHECK
}
