// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsIteration.h"
#include "InstancedActorsIndex.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsIteration)


FInstancedActorsIterationContext::~FInstancedActorsIterationContext()
{
	ensureMsgf(InstancesToRemove.IsEmpty(), TEXT("FInstancedActorsIterationContext destructing with dangling / un-flushed InstancesToRemove. FlushDeferredActions must be called once iteration completes, to perform deferred actions."));
	ensureMsgf(RemoveAllInstancesIAMs.IsEmpty(), TEXT("FInstancedActorsIterationContext destructing with dangling / un-flushed RemoveAllInstancesIAMs. FlushDeferredActions must be called once iteration completes, to perform deferred actions."));
}

void FInstancedActorsIterationContext::RemoveInstanceDeferred(const FInstancedActorsInstanceHandle& InstanceHandle)
{
	if (ensure(InstanceHandle.IsValid()))
	{
		InstancesToRemove.FindOrAdd(InstanceHandle.GetInstanceActorData()).Add(InstanceHandle.GetInstanceIndex());
	}
}

void FInstancedActorsIterationContext::RemoveAllInstancesDeferred(UInstancedActorsData& InstanceData)
{
	RemoveAllInstancesIADs.Add(&InstanceData);
}

void FInstancedActorsIterationContext::RemoveAllInstancesDeferred(AInstancedActorsManager& Manager)
{
	RemoveAllInstancesIAMs.Add(&Manager);
}

void FInstancedActorsIterationContext::FlushDeferredActions()
{
	for (auto& InstancesToRemoveItem : InstancesToRemove)
	{
		TObjectPtr<UInstancedActorsData> InstanceData = InstancesToRemoveItem.Key;
		if (ensure(IsValid(InstanceData)))
		{
			InstanceData->RuntimeRemoveInstances(MakeArrayView(InstancesToRemoveItem.Value));
		}
	}
	InstancesToRemove.Empty();

	for (TObjectPtr<UInstancedActorsData> InstanceData : RemoveAllInstancesIADs)
	{
		InstanceData->RuntimeRemoveAllInstances();
	}
	RemoveAllInstancesIADs.Empty();

	for (TObjectPtr<AInstancedActorsManager> Manager : RemoveAllInstancesIAMs)
	{
		Manager->RuntimeRemoveAllInstances();
	}
	RemoveAllInstancesIAMs.Empty();
}
