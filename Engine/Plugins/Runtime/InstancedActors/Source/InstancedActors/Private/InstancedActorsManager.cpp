// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"
#include "InstancedActorsComponent.h"
#include "InstancedActorsCustomVersion.h"
#include "InstancedActorsIteration.h"
#include "InstancedActorsModifierVolumeComponent.h"
#include "InstancedActorsSettingsTypes.h"
#include "InstancedActorsSettings.h"
#include "InstancedActorsSubsystem.h"
#include "InstancedActorsRepresentationActorManagement.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "Algo/NoneOf.h"
#include "Algo/Sort.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Dom/JsonObject.h"
#include "Engine/StaticMesh.h"
#include "EngineUtils.h"
#include "JsonDomBuilder.h"
#include "MassActorSubsystem.h"
#include "MassCommonFragments.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationSubsystem.h"
#include "Math/NumericLimits.h"

#include "Net/Iris/ReplicationSystem/ReplicationSystemUtil.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsManager)

DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Detailed Instance Count"), STAT_DetailedInstanceCount, STATGROUP_InstancedActorsRendering);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Medium Instance Count"), STAT_MediumInstanceCount, STATGROUP_InstancedActorsRendering);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Low Instance Count"), STAT_LowLODInstanceCount, STATGROUP_InstancedActorsRendering);
DECLARE_DWORD_ACCUMULATOR_STAT(TEXT("Off Instance Count"), STAT_OffInstanceCount, STATGROUP_InstancedActorsRendering);

namespace UE::InstancedActors
{
	namespace CVars
	{
		bool bDeferSpawnEntities = true;
		FAutoConsoleVariableRef CVarDeferSpawnEntities(
			TEXT("IA.DeferSpawnEntities"),
			bDeferSpawnEntities,
			TEXT("When enabled, Instanced Actor Managers will defer their entity spawning from BeginPlay to UInstancedActorsSubsystem::Tick.")
			TEXT("This can allow modifier volumes to pre-registered with IAMs for cheaper pre-spawn modification. Also allows for amortized ")
			TEXT("entity spawning with IA.DeferSpawnEntities.MaxTimePerTick"),
			ECVF_Default);

		bool bEnablePersistence = true;
		FAutoConsoleVariableRef CVarEnablePersistence(
			TEXT("IA.EnablePersistence"),
			bEnablePersistence,
			TEXT("When enabled, Instanced Actor destroyed state will persist in the saved game"),
			ECVF_Default);

		bool bInstanceCollisionsOnClient = true;
		FAutoConsoleVariableRef CVarCollisionsOnClient(
			TEXT("IA.InstanceCollisionsOnClient"),
			bInstanceCollisionsOnClient,
			TEXT("When enabled, Instanced Actor Manager ISMC's will be created with collision enabled on clients"),
			ECVF_Default);

		bool bInstanceCollisionsOnServer = true;
		FAutoConsoleVariableRef CVarInstanceCollisionsOnServer(
			TEXT("IA.InstanceCollisionsOnServer"),
			bInstanceCollisionsOnServer,
			TEXT("When enabled, Instanced Actor Manager ISMC's will be created with collision enabled on server"),
			ECVF_Default);

		int32 WorldPositionOffsetDisableDistance = 0;
		FAutoConsoleVariableRef CVarWorldPositionOffsetDisableDistance(
			TEXT("IA.WorldPositionOffsetDisableDistance"),
			WorldPositionOffsetDisableDistance,
			TEXT("WorldPositionOffsetDisableDistance which is set on the ISM components"),
			ECVF_Default);

		float WorldPositionOffsetDisableDistanceScale = 1.0;
		FAutoConsoleVariableRef CVarWorldPositionOffsetDisableDistanceScale(
			TEXT("IA.WorldPositionOffsetDisableDistanceScale"),
			WorldPositionOffsetDisableDistanceScale,
			TEXT("Scale the value of WorldPositionOffsetDisableDistance which is coming from IA.WorldPositionOffsetDisableDistance or set on the FInstancedActorsSettings"),
			ECVF_Default);

		bool bOverrideCastFarShadow = false;
		FAutoConsoleVariableRef CVarOverrideCastFarShadow(
			TEXT("IA.OverrideCastFarShadow"),
			bOverrideCastFarShadow,
			TEXT("Option to override 'Cast Far Shadow' flag to match Distance Field Shadows coverage."),
			ECVF_Default);

		bool bInstantHydrationViaPhysicsQueriesEnabled = true;
		FAutoConsoleVariableRef CVarInstantHydrationViaPhysicsQueriesEnabled(
			TEXT("IA.InstantHydrationViaPhysicsQueriesEnabled"),
			bInstantHydrationViaPhysicsQueriesEnabled,
			TEXT("Whether Instanced Actors react to engine physics queries like LWI does"),
			ECVF_Default);
	} // namespace CVars

#if !UE_BUILD_SHIPPING
	namespace Cmds
	{
		void CompactInstances(const TArray<FString>& Args, UWorld* World, FOutputDevice& Ar)
		{
			for (TActorIterator<AInstancedActorsManager> MangerIt(World); MangerIt; ++MangerIt)
			{
				AInstancedActorsManager* Manager = *MangerIt;
				check(Manager);

				Manager->CompactInstances(Ar);
			}
		}

		static FAutoConsoleCommand CompactInstancesCommand(
			TEXT("IA.CompactInstances"),
			TEXT("Removing instances from their managers simply invalidates their transforms, which ")
			TEXT("requires additional runtime cost to skip these invalidated instances during spawning. ")
			TEXT("This function fully removes these instances, compacting the instance lists and removing the ")
			TEXT("additional runtime cost. This does however result in reordering of instances, invalidating ")
			TEXT("any held FInstancedActorsInstanceHandle's"),
			FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateStatic(CompactInstances));
	}  // namespace Cmds
#endif // !UE_BUILD_SHIPPING

	namespace
	{
		FORCEINLINE bool IsValidInstanceTransform(const FTransform& InstanceTransform)
		{
			return !InstanceTransform.GetScale3D().IsZero();
		}
	} // anonymous

	template <>
	bool PassesBoundsTest<FSphere>(const FSphere& QueryBounds, EBoundsTestType BoundsTestType, const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform)
	{
		switch (BoundsTestType)
		{
		case EBoundsTestType::Intersect:
			{
				// Cheap test first
				if (QueryBounds.IsInside(InstanceTransform.GetLocation()))
				{
					return true;
				}
				
				const FBox& InstancedActorBounds = InstanceHandle.GetInstanceActorData()->GetCachedLocalBounds();
				const FSphere TransformedSphere = QueryBounds.TransformBy(InstanceTransform.Inverse());
				return FMath::SphereAABBIntersection(TransformedSphere, InstancedActorBounds);
			}
		case EBoundsTestType::Enclosed:
			{
				// Cheap test first
				if (!QueryBounds.IsInside(InstanceTransform.GetLocation()))
				{
					return false;
				}
				
				const FBox& InstancedActorBounds = InstanceHandle.GetInstanceActorData()->GetCachedLocalBounds();
				const FSphere TransformedSphere = QueryBounds.TransformBy(InstanceTransform.Inverse());
				return QueryBounds.IsInside(InstancedActorBounds.Min) && QueryBounds.IsInside(InstancedActorBounds.Max);
			}
		default:
			ensureMsgf(false, TEXT("Unexpected BoundsTestType: %i"), (int)BoundsTestType);
			break;
		}

		return false;
	}

	template <>
	bool PassesBoundsTest<FBox>(const FBox& QueryBounds, EBoundsTestType BoundsTestType, const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform)
	{
		switch (BoundsTestType)
		{
		case EBoundsTestType::Intersect:
			{
				// Cheap test first
				if (QueryBounds.IsInside(InstanceTransform.GetLocation()))
				{
					return true;
				}
				
				const FBox InstancedActorBounds = InstanceHandle.GetInstanceActorData()->GetCachedLocalBounds().TransformBy(InstanceTransform);
				return QueryBounds.Intersect(InstancedActorBounds);
			}
		case EBoundsTestType::Enclosed:
			{
				// Cheap test first
				if (!QueryBounds.IsInside(InstanceTransform.GetLocation()))
				{
					return false;
				}
				
				const FBox InstancedActorBounds = InstanceHandle.GetInstanceActorData()->GetCachedLocalBounds().TransformBy(InstanceTransform);
				return QueryBounds.IsInside(InstancedActorBounds);
			}
		default:
			ensureMsgf(false, TEXT("Unexpected BoundsTestType: %i"), (int)BoundsTestType);
			break;
		}
		
		return false;
	}

} // namespace InstancedActors

//-----------------------------------------------------------------------------
// AInstancedActorsManager
//-----------------------------------------------------------------------------
AInstancedActorsManager::AInstancedActorsManager()
{
	InstancedActorsDataClass = UInstancedActorsData::StaticClass();
	bReplicateUsingRegisteredSubObjectList = true;
	bReplicates = true;
	SetNetDormancy(DORM_DormantAll);
	// @todo need default implementation of GUID and set it here via SetSavedActorGUID
}

void AInstancedActorsManager::OnReplicationStartedForIris(const FOnReplicationStartedParams& Params)
{
	Super::OnReplicationStartedForIris(Params);

	// This actor is configured to use spatial prioritization in Iris. If it's too far from
	// a player's position it will replicate to their client less frequently. This becomes a 
	// problem when destroying instanced actors because the client needs to receive both
	// the actor destruction notification and instanced actor destruction notification
	// at the same time. If the aren't received at the same time then the instanced actor's
	// mesh may still be visible for a period of time after destroying the actor.
	//
	// Fixing the priority to 1.0 means that it will always have the maximum priority and
	// replicated at the maximum configured frequency.
	UE::Net::FReplicationSystemUtil::SetStaticPriority(this, 1.0f);
}

void AInstancedActorsManager::BeginPlay()
{
	UWorld* World = GetWorld();
	check(World);

#if WITH_EDITORONLY_DATA
	if (bIsEditorOnlyActor && World && World->IsPlayInEditor())
	{
		Destroy();
		return;
	}
#endif

	Super::BeginPlay();

	TRACE_CPUPROFILER_EVENT_SCOPE(AInstancedActorsManager::BeginPlay);

	UE_LOGF(LogInstancedActors, Verbose, "%ls (%d instances) BeginPlay", *GetPathName(), GetNumValidInstances());
	UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>();
	if (!ensure(EntitySubsystem))
	{
		UE_LOGF(LogInstancedActors, Error, "%ls BeginPlay fail due to missing UMassEntitySubsystem", *GetPathName());
		return;
	}
	MassEntityManager = EntitySubsystem->GetMutableEntityManager().AsShared();

	// the main reason for not supporting pre-BeginPlay registration is that we cache the InstanceBounds here and the
	// bounds are used during manager's registraiton to place it on the 2d hashmap
	checkf(ManagerHandle.IsValid() == false, TEXT("We don't expect IAMs to be registered before their BeginPlay."));

	// Cache world instance bounds
	//
	// Note: This must be done before InstancedActorSubsystem->AddManager bewlow, as this is used to
	// spatially index the manager.
	InstanceBounds = CalculateLocalInstanceBounds().TransformBy(GetActorTransform());

	// Register with IA subsystem if it's available already, otherwise the subsystem will
	// collect this manager when it initializes later and call OnAddedToSubsystem
	UInstancedActorsSubsystem* PreinitializedInstancedActorSubsystem = UE::InstancedActors::Utils::GetInstancedActorsSubsystem(*World);
	if (PreinitializedInstancedActorSubsystem)
	{
		// Register manager with subsystem now, which will immediately call OnAddedToSubsystem
		PreinitializedInstancedActorSubsystem->AddManager(*this);
	}
}

void AInstancedActorsManager::OnAddedToSubsystem(UInstancedActorsSubsystem& InInstancedActorSubsystem, FInstancedActorsManagerHandle InManagerHandle)
{
	if (IsValid(InstancedActorSubsystem) && (InstancedActorSubsystem != &InInstancedActorSubsystem))
	{
		// we need to unregister from the previous subsystem first
		DespawnAllEntities();
		// and let registered modifiers go
		RemoveAllModifierVolumes();
		
		// Since the old IASubsystem is part of a different exemplar world, to be safe we should release the IADs' references to their
		// exemplar actors, since they were spawned under that old world
		for (TObjectPtr<UInstancedActorsData>& InstanceData : PerActorClassInstanceData)
		{
			check(InstanceData);
			InstanceData->ExemplarActorData.Reset();
		}

		InstancedActorSubsystem->RemoveManager(ManagerHandle);
		InstancedActorSubsystem = nullptr;
		ManagerHandle.Reset();
	}
	
	checkf(!IsValid(InstancedActorSubsystem), TEXT("Manager %s has already been added to a %s"), *GetPathName(), *InstancedActorSubsystem->GetName());
	checkf(!ManagerHandle.IsValid(), TEXT("Manager %s has already been added to a UInstancedActorsSubsystem"), *GetPathName());
	check(InManagerHandle.IsValid());

	InstancedActorSubsystem = &InInstancedActorSubsystem;
	ManagerHandle = InManagerHandle;

	// Find overlapping modifiers that have already been registered with InstancedActorSubsystem. Modifier volumes that register after this
	// will latently add their modifiers to overlapping managers in their UInstancedActorsModifierVolumeComponent::OnAddedToSubsystem
	InstancedActorSubsystem->ForEachModifierVolume(InstanceBounds, [this](UInstancedActorsModifierVolumeComponent& ModifierVolume)
		{
			AddModifierVolume(ModifierVolume);
			return true; 
		});

	if (UE::InstancedActors::CVars::bDeferSpawnEntities)
	{
		InstancedActorSubsystem->RequestDeferredSpawnEntities(ManagerHandle);
	}
	else
	{
		InitializeModifyAndSpawnEntities();
	}
}

void AInstancedActorsManager::InitializeModifyAndSpawnEntities()
{
	check(IsValid(InstancedActorSubsystem));

	if (InstancedActorLocationQuery.IsInitialized() == false && ensure(MassEntityManager))
	{
		InstancedActorLocationQuery.Initialize(MassEntityManager.ToSharedRef());
		InstancedActorLocationQuery.AddRequirement<FInstancedActorsFragment>(EMassFragmentAccess::ReadOnly);
		InstancedActorLocationQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	}

	// Debug draw managers as they load
#if WITH_INSTANCEDACTORS_DEBUG
	UE::InstancedActors::Debug::DebugDrawManager(UE::InstancedActors::Debug::CVars::DebugManagerLoading, *this);
	UE::InstancedActors::Debug::DebugDrawAllInstanceLocations(UE::InstancedActors::Debug::CVars::DebugInstanceLoading, ELogVerbosity::Verbose, *this, TOptional<FColor>(), /*LogOwner*/ this);
	UE::InstancedActors::Debug::LogInstanceCountsOnScreen(UE::InstancedActors::Debug::CVars::DebugInstanceLoading, *this);
#endif // WITH_INSTANCEDACTORS_DEBUG

	// Initialize all PerActorClassInstanceData before executing modifiers
	// - Compile and cache settings (ensures they're accessible by modifiers)
	// - Add default visualization
	// - Create entity template
	for (TObjectPtr<UInstancedActorsData>& InstanceData : PerActorClassInstanceData)
	{
		check(InstanceData);
		InstanceData->Initialize();
	}

	// Run pending modifiers that can run pre-entity spawn
	TryRunPendingModifiers();

	// SpawnEntities for all PerActorClassInstanceData
	for (TObjectPtr<UInstancedActorsData>& InstanceData : PerActorClassInstanceData)
	{
		check(InstanceData);
		InstanceData->SpawnEntities();
	}
	bHasSpawnedEntities = true;

	// Try running still-pending modifiers that require entities / must run after SpawnEntities
	TryRunPendingModifiers();

	// If InitializeModifyAndSpawnEntities was deferred, we may have already received persistence deltas to apply so we try and apply
	// any here, now that we've spawned entities to apply them to.
	for (TObjectPtr<UInstancedActorsData>& InstanceData : PerActorClassInstanceData)
	{
		check(InstanceData);
		InstanceData->ApplyInstanceDeltas();
	}

	// Run virtual callback to let inheritors know that we've initialized/modified/spawned entities
	OnInitializeModifyAndSpawnEntities();
}

void AInstancedActorsManager::EndPlay(const EEndPlayReason::Type EndPlayReason)
{
	DespawnAllEntities();

	for (TObjectPtr<UInstancedActorsData>& InstanceData : PerActorClassInstanceData)
	{
		check(InstanceData);
		InstanceData->Deinitialize();
	}

	// Deregister with UInstancedActorsSubsystem
	if (InstancedActorSubsystem)
	{
		check(ManagerHandle.IsValid());
		InstancedActorSubsystem->RemoveManager(ManagerHandle);
		ManagerHandle.Reset();
		InstancedActorSubsystem = nullptr;
	}
	check(!ManagerHandle.IsValid());

	// Remove modifier volumes
	for (TWeakObjectPtr<UInstancedActorsModifierVolumeComponent>& WeakModifierVolume : ModifierVolumes)
	{
		if (UInstancedActorsModifierVolumeComponent* ModifierVolume = WeakModifierVolume.Get())
		{
			ModifierVolume->OnRemovedFromManager(*this);
		}
	}
	ModifierVolumes.Reset();
	PendingModifierVolumes.Reset();
	PendingModifierVolumeModifiers.Reset();

	MassEntityManager.Reset();

	Super::EndPlay(EndPlayReason);
}

bool AInstancedActorsManager::IsHLODRelevant() const
{
	return false;
}

void AInstancedActorsManager::PostLoad()
{
	Super::PostLoad();

	SetupLoadedInstances();
}

void AInstancedActorsManager::SetupLoadedInstances()
{
	if (bHasSetupLoadedInstances)
	{
		return;
	}
	
	bHasSetupLoadedInstances = true;

	// @todo need default implementation of GUID and set it here via SetSavedActorGUID

	for (UInstancedActorsData* InstanceData : PerActorClassInstanceData)
	{
		REDIRECT_OBJECT_TO_VLOG(InstanceData, this);
		// This should be added on client and server, this needs to be added on the client to allow replays to work.
		AddReplicatedSubObject(InstanceData);
	}
}

void AInstancedActorsManager::Serialize(FStructuredArchive::FRecord Record)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AInstancedActorsManager::Serialize);

	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();

	UnderlyingArchive.UsingCustomVersion(FInstancedActorsCustomVersion::GUID);

	// Persistence data serialization
	if (UnderlyingArchive.IsSaveGame())
	{
		if (!UE::InstancedActors::CVars::bEnablePersistence)
		{
			return;
		}

		check(HasAuthority());

		if (!UnderlyingArchive.IsLoading() && !UnderlyingArchive.IsSaving() && UnderlyingArchive.IsObjectReferenceCollector())
		{
			// Early out for SaveGame object reference collection as we don't serialize out any object refs in IAM persistence data
			// @todo Allow a mechanism for UInstancedActorsComponent's to do so if they need (they don't currently)
			return;
		}

		UE_CLOGF(UnderlyingArchive.IsLoading(), LogInstancedActors, Verbose, "%ls loading persistent data", *GetPathName());
		UE_CLOGF(!UnderlyingArchive.IsLoading(), LogInstancedActors, Verbose, "%ls saving persistent data", *GetPathName());

		if (UnderlyingArchive.IsLoading())
		{
			FlushNetDormancy();
		}

		// Store / retrieve world real time at serialization
		UWorld* World = GetWorld();
		check(World);
		const FDateTime TimeNow = FDateTime::UtcNow();
		FDateTime SerializedTime = TimeNow;
		Record << SA_VALUE(TEXT("Time"), SerializedTime);
		
		int64 TimeDelta = UnderlyingArchive.IsLoading() ? (TimeNow - SerializedTime).GetTotalSeconds() : 0;
		if (!ensure(TimeDelta >= 0))
		{
			TimeDelta = 0;
		}

		// Seialize each IAD
		int32 NumInstanceDatas = PerActorClassInstanceData.Num();
		FStructuredArchiveArray InstanceDataArray = Record.EnterArray(TEXT("InstanceData"), NumInstanceDatas);

		for (int32 InstanceDataIndex = 0; InstanceDataIndex < NumInstanceDatas; ++InstanceDataIndex)
		{
			FStructuredArchiveRecord InstanceDataRecord = InstanceDataArray.EnterElement().EnterRecord();

			UInstancedActorsData* InstanceData = nullptr;
			if (UnderlyingArchive.IsLoading())
			{
				// Find IAD for ID
				uint16 InstanceDataID;
				InstanceDataRecord << SA_VALUE(TEXT("ID"), InstanceDataID);
				InstanceData = FindInstanceDataByID(InstanceDataID);

				if (InstanceData == nullptr)
				{
					UE_LOGF(LogInstancedActors, Warning, "%ls - no IAD found with ID %u to restore persistent data. Data will be ignored and expunged on re-save", *GetPathName(), InstanceDataID);
				}
			}
			else
			{
				// Save ID for later matchup
				check(PerActorClassInstanceData.IsValidIndex(InstanceDataIndex));
				InstanceData = PerActorClassInstanceData[InstanceDataIndex];
				check(IsValid(InstanceData));
				InstanceDataRecord << SA_VALUE(TEXT("ID"), InstanceData->ID);
			}

			// Serialize / deserialize IAD persistence data
			// Note: This is performed even if InstanceData = nullptr to seek the archive past the saved data
			SerializeInstancePersistenceData(InstanceDataRecord, InstanceData, TimeDelta);

			if (UnderlyingArchive.IsError())
			{
				return;
			}
		}
	}
	else
	{
		Super::Serialize(Record);
	}
}

void AInstancedActorsManager::SerializeInstancePersistenceData(FStructuredArchive::FRecord Record, UInstancedActorsData* InstanceData, int64 TimeDelta) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(AInstancedActorsManager::SerializeInstancePersistenceData);

	FArchive& UnderlyingArchive = Record.GetUnderlyingArchive();
	check(UnderlyingArchive.IsSaveGame());

	// Note: InstanceData may be nullptr when loading, if there were persitence records saved for IAD's that have since been removed. In this case
	// we still need to ensure the archive is seeked past this data to safely ignore it.
	if (InstanceData == nullptr)
	{
		check(UnderlyingArchive.IsLoading());
	}

	// Note: Rather than serializing a flat array of InstanceDelta's directly as an Array Of Structs, we instead serialize a Struct Of Arrays
	//       to optimize for the common case where we have lots of destroyed instances where rather than storing an array of instances with
	//       true / false for destroyed state, we can just have an array of destroyed instane indices, factoring out the extra byte for destroyed
	//       state.
	// @todo: In hindsight we could / should just intrinisically have separate arrays / 'delta lists' of Destroyed Instances and Lifecycle Changes.

	// Destroyed instance indices
	int32 NumDestroyedInstances = InstanceData ? InstanceData->InstanceDeltas.GetNumDestroyedInstanceDeltas() : 0;
	FStructuredArchive::FArray DestroyedInstancesArray = Record.EnterArray(TEXT("DestroyedInstances"), NumDestroyedInstances);

	if (UnderlyingArchive.IsLoading())
	{
		for (int32 ArrayIndex = 0; ArrayIndex < NumDestroyedInstances; ++ArrayIndex)
		{
			const int32 NumInstances = InstanceData ? InstanceData->GetNumInstances() : 0;
			if (!ensureMsgf(InstanceData && ArrayIndex < NumInstances,
				TEXT("Attempting to destroy more instances (%d) than actually exist (%d). Aborting corrupted persistence archive read. Persistence data may be lost as a result."),
				NumDestroyedInstances, NumInstances))
			{
				UnderlyingArchive.SetError();
				return;
			}

			FInstancedActorsInstanceIndex DestroyedInstanceIndex;
			DestroyedInstancesArray << DestroyedInstanceIndex;

			if (!ensureMsgf(!UnderlyingArchive.GetError(), TEXT("Error reading DestroyedInstancesArray element. Aborting corrupted persistence archive read. Persistence data may be lost as a result.")))
			{
				return;
			}

			if (InstanceData)
			{
				InstanceData->InstanceDeltas.SetInstanceDestroyed(DestroyedInstanceIndex);
			}
		}
	}
	else
	{
		check(InstanceData);

		int32 NumDestroyedInstancesWritten = 0;
		const TArray<FInstancedActorsDelta>& InstanceDeltas = InstanceData->InstanceDeltas.GetInstanceDeltas();
		for (const FInstancedActorsDelta& Delta : InstanceDeltas)
		{
			if (Delta.IsDestroyed())
			{
				FInstancedActorsInstanceIndex DestroyedInstanceIndex = Delta.GetInstanceIndex();
				DestroyedInstancesArray << DestroyedInstanceIndex;
				++NumDestroyedInstancesWritten;
			}
		}
		ensureMsgf(NumDestroyedInstancesWritten == NumDestroyedInstances, TEXT("AInstancedActorsManager::SerializeInstancePersistenceData: Expectations are for the Number of Instances destroyed to match the number of Instances marked to be destroyed in serialization"));
	}

	// Allow UInstancedActorComponents (IAC's) to extend persistence
	//
	// IAC persistence entries are written out with an ID & size header so they can be matched up with their
	// serialization implementation when read back later or safely skipped if the IAC has since been removed
	// from ActorClass
	TMap<uint32, UInstancedActorsComponent*, TInlineSetAllocator<2>> IACsByPersistenceID;
	TMap<UInstancedActorsComponent*, uint32, TInlineSetAllocator<2>> IACPersistenceIDs;
	int32 NumPersistedIACs = 0;
	if (InstanceData)
	{
		TSharedPtr<UE::InstancedActors::FExemplarActorData> ExemplarActorData = GetInstancedActorSubsystemChecked().GetOrCreateExemplarActor(InstanceData->ActorClass);
		const AActor* const ExemplarActor = ExemplarActorData->Actor.Get();
		check(ExemplarActor);
		ExemplarActor->ForEachComponent<UInstancedActorsComponent>(/*bIncludeFromChildActors*/ false, [&](UInstancedActorsComponent* InstancedActorComponent)
			{
				// Skip IAC's that don't want / need serialization. When writing this ensures we don't waste space by writing empty entries.
				if (InstancedActorComponent->ShouldSerializeInstancePersistenceData(UnderlyingArchive, InstanceData, TimeDelta))
				{
					const uint32 IACPersistenceID = InstancedActorComponent->GetInstancePersistenceDataID();
					if (ensureMsgf(IACPersistenceID != 0, TEXT("UInstancedActorsComponent classes implementing ShouldSerializeInstancePersistenceData (%s) must also implement GetInstancePersistenceDataID and return a non-zero value"), *InstancedActorComponent->GetClass()->GetPathName()))
					{
						IACsByPersistenceID.Add(IACPersistenceID, InstancedActorComponent);
						IACPersistenceIDs.Add(InstancedActorComponent, IACPersistenceID);
						++NumPersistedIACs;
					}
				} });
	}
	FStructuredArchiveArray InstancedActorComponentDataArray = Record.EnterArray(TEXT("InstancedActorComponentData"), NumPersistedIACs);
	if (UnderlyingArchive.IsLoading())
	{
		for (int32 IACIndex = 0; IACIndex < NumPersistedIACs; ++IACIndex)
		{
			FStructuredArchiveRecord InstancedActorComponentDataRecord = InstancedActorComponentDataArray.EnterElement().EnterRecord();

			// Find matching IAC by ID
			uint32 IACPersistenceID = 0;
			int32 IACPersistenceDataSize = -1;
			InstancedActorComponentDataRecord << SA_VALUE(TEXT("IACPersistenceID"), IACPersistenceID);
			InstancedActorComponentDataRecord << SA_VALUE(TEXT("IACPersistenceDataSize"), IACPersistenceDataSize);

			// Read data block with matching IAC
			if (UInstancedActorsComponent** InstancedActorComponent = IACsByPersistenceID.Find(IACPersistenceID))
			{
				check(InstanceData);

				const int64 IACPersistenceDataStartOffset = UnderlyingArchive.Tell();

				// Deserialize IAC data
				(*InstancedActorComponent)->SerializeInstancePersistenceData(InstancedActorComponentDataRecord, InstanceData, TimeDelta);

				const int64 IACPersistenceDataEndOffset = UnderlyingArchive.Tell();
				const int32 IACPersistenceDataSizeRead = IntCastChecked<int32>(IACPersistenceDataEndOffset - IACPersistenceDataStartOffset);

				// Make sure we read the full block
				if (!ensure(IACPersistenceDataSizeRead == IACPersistenceDataSize))
				{
					UnderlyingArchive.Seek(IACPersistenceDataStartOffset + IACPersistenceDataSize);
				}
			}
			else
			{
				if (!ensureMsgf(IACPersistenceDataSize >= 0, TEXT("Expected valid positive data size in bytes >= 0. Found: %d. Aborting persistence archive read. Persistence data may be lost as a result."), IACPersistenceDataSize))
				{
					UnderlyingArchive.SetError();
					return;
				}

				// Skip IAC data block which we no longer have a matching IAC to read with
				UE_CLOGF(!InstanceData, LogInstancedActors, Error, "No UInstancedActorComponent with PersistenceID %u found due to no IAD found to get exemplar for! Skipping this IAC data block which will be lost on resave!", IACPersistenceID);
				UE_CLOGF(InstanceData, LogInstancedActors, Error, "No UInstancedActorComponent with PersistenceID %u found in %ls ExemplarActor. Skipping this IAC data block which will be lost on resave!", IACPersistenceID, *InstanceData->ActorClass->GetPathName());
				UnderlyingArchive.Seek(UnderlyingArchive.Tell() + IACPersistenceDataSize);
			}

			if (UnderlyingArchive.IsError())
			{
				return;
			}
		}
	}
	else
	{
		int32 NumPersistedIACWritten = 0;
		for (const auto& IACPersistenceIDsItem : IACPersistenceIDs)
		{
			UInstancedActorsComponent* InstancedActorComponent = IACPersistenceIDsItem.Key;
			uint32 IACPersistenceID = IACPersistenceIDsItem.Value;

			FStructuredArchiveRecord InstancedActorComponentDataRecord = InstancedActorComponentDataArray.EnterElement().EnterRecord();

			// Write ID and size header to match up the right IAC type on load later, or seek past the size
			InstancedActorComponentDataRecord << SA_VALUE(TEXT("IACPersistenceID"), IACPersistenceID);

			// Write placeholder size for now which we'll seek back to and update
			const int64 IACPersistenceDataSizeOffset = UnderlyingArchive.Tell();
			int32 IACPersistenceDataSize = 0;
			InstancedActorComponentDataRecord << SA_VALUE(TEXT("IACPersistenceDataSize"), IACPersistenceDataSize);

			// Write the IAC data
			const int64 IACPersistenceDataStartOffset = UnderlyingArchive.Tell();
			InstancedActorComponent->SerializeInstancePersistenceData(InstancedActorComponentDataRecord, InstanceData, TimeDelta);
			const int64 IACPersistenceDataEndOffset = UnderlyingArchive.Tell();
			IACPersistenceDataSize = IntCastChecked<int32>(IACPersistenceDataEndOffset - IACPersistenceDataStartOffset);
			ensure(IACPersistenceDataSize >= 0);

			// Seek back and re-write the size now we know what it is
			if (!UnderlyingArchive.IsTextFormat())
			{
				UnderlyingArchive.Seek(IACPersistenceDataSizeOffset);
				UnderlyingArchive << IACPersistenceDataSize;
				UnderlyingArchive.Seek(IACPersistenceDataEndOffset);
			}

			++NumPersistedIACWritten;
		}
		check(NumPersistedIACWritten == NumPersistedIACs);
	}
}

#if WITH_EDITOR
void AInstancedActorsManager::GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const
{
	UWorld* World = GetWorld();
	check(World);

	if (World->IsPartitionedWorld())
	{
		UActorPartitionSubsystem::FCellCoord CellCoord = UActorPartitionSubsystem::FCellCoord::GetCellCoord(GetActorLocation(), GetLevel(), GetGridSize());
		OutRuntimeBounds = OutEditorBounds = UActorPartitionSubsystem::FCellCoord::GetCellBounds(CellCoord, GetGridSize());
	}
	else
	{
		OutRuntimeBounds = OutEditorBounds = FBox(FVector(-HALF_WORLD_MAX), FVector(HALF_WORLD_MAX));
	}
}

uint32 AInstancedActorsManager::GetDefaultGridSize(UWorld* InWorld) const
{
	return GetDefault<UInstancedActorsProjectSettings>()->GridSize;
}

FGuid AInstancedActorsManager::GetGridGuid() const
{
	return ManagerGridGuid;
}

void AInstancedActorsManager::SetGridGuid(const FGuid& InGuid)
{
	ManagerGridGuid = InGuid;
}

UInstancedActorsData* AInstancedActorsManager::CreateNextInstanceActorData(TSubclassOf<AActor> ActorClass, const FInstancedActorsTagSet& AdditionalInstanceTags)
{
	check(InstancedActorsDataClass);

	const FString InstanceDataNameStr = FString::Printf(TEXT("InstancedActorsData_%s"), *ActorClass->GetFName().ToString());
	const FName UniqueName = MakeUniqueObjectName(this, InstancedActorsDataClass, FName(InstanceDataNameStr));
	UInstancedActorsData* NewInstanceData = NewObject<UInstancedActorsData>(this, InstancedActorsDataClass, UniqueName);
	// @todo it's conceivable the NextInstanceDataID will overflow. We need to use some handle system in place instead. 
	NewInstanceData->ID = NextInstanceDataID++;
	NewInstanceData->ActorClass = ActorClass;
	NewInstanceData->AdditionalTags = AdditionalInstanceTags;
	check(Algo::NoneOf(PerActorClassInstanceData, [NewInstanceData](UInstancedActorsData* InstanceData)
		{
			return InstanceData->ID == NewInstanceData->ID;
		}));

	return NewInstanceData;
}

UInstancedActorsData& AInstancedActorsManager::GetOrCreateActorInstanceData(TSubclassOf<AActor> ActorClass, const FInstancedActorsTagSet& AdditionalInstanceTags, bool bCreateEditorPreviewISMCs)
{
	checkf(HasActorBegunPlay() == false, TEXT("AInstancedActorsManager doesn't yet support runtime addition of instances"));

	TObjectPtr<UInstancedActorsData>* InstanceData = PerActorClassInstanceData.FindByPredicate([ActorClass, AdditionalInstanceTags](TObjectPtr<UInstancedActorsData> InstanceData)
		{ 
			return InstanceData->ActorClass == ActorClass && InstanceData->AdditionalTags == AdditionalInstanceTags; 
		});

	if (InstanceData != nullptr)
	{
		check(IsValid(*InstanceData));
		return **InstanceData;
	}

	UInstancedActorsData* NewInstanceData = CreateNextInstanceActorData(ActorClass, AdditionalInstanceTags);
	PerActorClassInstanceData.Add(NewInstanceData);

	if (bCreateEditorPreviewISMCs)
	{
		// Get or create exemplar actor to derive entities from
		UWorld* World = GetWorld();
		check(World);
		UInstancedActorsSubsystem* EditorInstancedActorSubsystem = UE::InstancedActors::Utils::GetInstancedActorsSubsystem(*World);
		check(EditorInstancedActorSubsystem);
		// @todo For performance reasons it would be helpful to be able to cache this into UInstancedActorsData::ExemplarActorData
		// This would mean that undo-ing a conversion of actors to IAM in Editor will not clear or destroy the exemplar actor/map entry,
		// which is the reason for the sanity-check of the actor ptr in UInstancedActorsSubsystem::GetOrCreateExemplarActor().
		// We need to also investigate why converting the IAM to actors (destroying the IAM) does not seem to trigger GC on the UInstancedActorsData
		TSharedPtr<UE::InstancedActors::FExemplarActorData> ExemplarActorData = EditorInstancedActorSubsystem->GetOrCreateExemplarActor(ActorClass);
		const AActor* const ExemplarActor = ExemplarActorData->Actor.Get();
		check(ExemplarActor);

		// Create stand-in 'editor only' ISMC's to preview instances in the level editor. These will be stripped
		// out at cook time and in PostLoad for game worlds (PIE)
		FInstancedActorsVisualizationDesc DefaultVisualiation = EditorInstancedActorSubsystem->CreateVisualDescriptionFromActor(*ExemplarActor);
		CreateISMComponents(DefaultVisualiation, NewInstanceData->SharedSettings, NewInstanceData->EditorPreviewISMComponents, /*bEditorPreviewISMCs*/ true);

		// Add EditorPreviewISMComponents to the known ISMComponent-to-InstanceData map (ISMComponentToInstanceDataMap)
		// so that we can identify Instanced Actors instances by ISM instance id coming from one of the EditorPreviewISMComponents
		// (like when the instance is clicked in the editor).
		// @see ActorInstanceHandleFromFSMInstanceId
		RegisterInstanceDatasComponents(*NewInstanceData, NewInstanceData->EditorPreviewISMComponents);

		// Allow subclasses to customize the freshly created preview ISMCs (e.g. apply material overrides).
		NewInstanceData->OnEditorPreviewISMCsCreated();
	}

	// Cache asset bounds for use during instance population
	NewInstanceData->CachedLocalBounds = CalculateBounds(ActorClass);
	ensure(NewInstanceData->CachedLocalBounds.IsValid);

	return *NewInstanceData;
}

void AInstancedActorsManager::PreRegisterAllComponents()
{
	Super::PreRegisterAllComponents();

	for (UInstancedActorsData* InstanceData : PerActorClassInstanceData)
	{
		// Modify the ISMCs here since the components might have been serialized before this change.
		for (TObjectPtr<UInstancedStaticMeshComponent> ISMComponent : InstanceData->EditorPreviewISMComponents)
		{
			if (ISMComponent == nullptr)
			{
				ensureAlwaysMsgf(false, TEXT("UInstancedActorsData has a null EditorPreviewISMComponents element: %s"), *InstanceData->GetDebugName(/*bCompact*/ false));
				continue;
			}

			SetUpEditorPreviewISMComponent(ISMComponent);
		}

		// Add EditorPreviewISMComponents to the known ISMComponent-to-InstanceData map (ISMComponentToInstanceDataMap)
		// so that we can identify Instanced Actors instances by ISM instance id coming from one of the EditorPreviewISMComponents
		// (like when the instance is clicked in the editor). 
		// @see ActorInstanceHandleFromFSMInstanceId 
		RegisterInstanceDatasComponents(*InstanceData, InstanceData->EditorPreviewISMComponents);
	}
}

FInstancedActorsInstanceHandle AInstancedActorsManager::AddActorInstance(TSubclassOf<AActor> ActorClass, FTransform InstanceTransform, bool bWorldSpace, const FInstancedActorsTagSet& AdditionalInstanceTags)
{
	UInstancedActorsData& InstanceData = GetOrCreateActorInstanceData(ActorClass, AdditionalInstanceTags);
	return InstanceData.AddInstance(InstanceTransform, bWorldSpace);
}

bool AInstancedActorsManager::RemoveActorInstance(const FInstancedActorsInstanceHandle& InstanceToRemove)
{
	if (ensureMsgf(IsValidInstance(InstanceToRemove), TEXT("DeleteActorInstance called for invalid InstanceHandle (%s) for this manager (%s)"), *InstanceToRemove.GetDebugName(), *GetName()))
	{
		return InstanceToRemove.InstancedActorData->RemoveInstance(InstanceToRemove);
	}

	return false;
}
#endif // WITH_EDITOR

UInstancedActorsData* AInstancedActorsManager::FindInstanceDataByID(uint16 InstanceDataID) const
{
	// Shortcut for the usual case where no IAD's have been deleted
	if (PerActorClassInstanceData.IsValidIndex(InstanceDataID))
	{
		UInstancedActorsData* InstanceData = PerActorClassInstanceData[InstanceDataID];
		check(IsValid(InstanceData));

		if (InstanceData->ID == InstanceDataID)
		{
			return InstanceData;
		}
	}

	// An IAD must have been removed, affecting the array ordering, brute force search instead
	for (UInstancedActorsData* InstanceData : PerActorClassInstanceData)
	{
		check(IsValid(InstanceData));

		if (InstanceData->ID == InstanceDataID)
		{
			return InstanceData;
		}
	}

	return nullptr;
}

void AInstancedActorsManager::RuntimeRemoveAllInstances()
{
	for (TObjectPtr<UInstancedActorsData> InstanceData : PerActorClassInstanceData)
	{
		InstanceData->RuntimeRemoveAllInstances();
	}
}

void AInstancedActorsManager::DespawnAllEntities()
{
	if (HasSpawnedEntities())
	{
		// DespawnEntities for all PerActorClassInstanceData
		for (TObjectPtr<UInstancedActorsData> InstanceData : PerActorClassInstanceData)
		{
			InstanceData->DespawnEntities();
		}
		bHasSpawnedEntities = false;
	}
}

void AInstancedActorsManager::DespawnAllEntities(bool /*bIsSwitchingSubsystems*/)
{
	DespawnAllEntities();
}

bool AInstancedActorsManager::ForEachInstance(FInstanceOperationFunc Operation) const
{
	FScopedInstancedActorsIterationContext IterationContext;

	return ForEachInstance(Operation, IterationContext);
}

bool AInstancedActorsManager::ForEachInstance(FInstanceOperationFunc Operation, FInstancedActorsIterationContext& IterationContext, TOptional<FInstancedActorDataPredicateFunc> InstancedActorDataPredicate) const
{
	FInstancedActorsInstanceHandle InstanceHandle;

	bool bContinue = true;

	// After SpawnEntities, we must operate on Mass entities
	if (HasSpawnedEntities())
	{
		check(MassEntityManager.IsValid());

		for (TObjectPtr<UInstancedActorsData> InstanceData : PerActorClassInstanceData)
		{
			// InstancedActorDataPredicate filter PerActorClassInstanceData
			check(IsValid(InstanceData));
			if (InstancedActorDataPredicate.IsSet())
			{
				const bool bPassedPredicate = ::Invoke(*InstancedActorDataPredicate, *InstanceData);
				if (!bPassedPredicate)
				{
					continue;
				}
			}

			InstanceHandle.InstancedActorData = InstanceData;

			TArray<FMassArchetypeEntityCollection> EntityCollections;
			UE::Mass::Utils::CreateEntityCollections(*MassEntityManager, InstanceData->Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);

			FMassExecutionContext ExecutionContext(*MassEntityManager);
			InstancedActorLocationQuery.ForEachEntityChunkInCollections(EntityCollections, ExecutionContext, [&IterationContext, &InstanceHandle, &Operation, &bContinue](FMassExecutionContext& Context)
				{
					if (!bContinue)
					{
						return;
					}

					TConstArrayView<FInstancedActorsFragment> InstancedActorFragments = Context.GetFragmentView<FInstancedActorsFragment>();
					TConstArrayView<FTransformFragment> TransformsFragments = Context.GetFragmentView<FTransformFragment>();
					for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
					{
						const FTransformFragment& TransformFragment = TransformsFragments[EntityIt];

						const FInstancedActorsFragment& InstancedActorFragment = InstancedActorFragments[EntityIt];
						InstanceHandle.Index = InstancedActorFragment.InstanceIndex;

						// Execute operation
						bContinue = Operation(InstanceHandle, TransformFragment.GetTransform(), IterationContext);
						if (!bContinue)
						{
							return;
						}
					}
				});

			if (!bContinue)
			{
				break;
			}
		}
	}
	// Before begin play, iterate source InstanceTransforms list
	else
	{
		const FVector ManagerLocation = GetActorLocation();
		const FTransform& ManagerTransform = GetActorTransform();
		const bool bApplyManagerTranslationOnly = (GetActorQuat().IsIdentity() && GetActorScale().Equals(FVector::OneVector));

		for (TObjectPtr<UInstancedActorsData> InstanceData : PerActorClassInstanceData)
		{
			// InstancedActorDataPredicate filter PerActorClassInstanceData
			check(IsValid(InstanceData));
			if (InstancedActorDataPredicate.IsSet())
			{
				const bool bPassedPredicate = ::Invoke(*InstancedActorDataPredicate, *InstanceData);
				if (!bPassedPredicate)
				{
					continue;
				}
			}

			InstanceHandle.InstancedActorData = InstanceData;

			uint16 InstanceIndex = 0;
			for (const FTransform& InstanceTransform : InstanceData->InstanceTransforms)
			{
				if (UE::InstancedActors::IsValidInstanceTransform(InstanceTransform))
				{
					InstanceHandle.Index = FInstancedActorsInstanceIndex(InstanceIndex);

					// Compute world space transform
					FTransform WorldSpaceInstanceTransform = InstanceTransform;
					if (bApplyManagerTranslationOnly)
					{
						WorldSpaceInstanceTransform.AddToTranslation(ManagerLocation);
					}
					else
					{
						WorldSpaceInstanceTransform *= ManagerTransform;
					}

					// Execute operation
					bContinue = Operation(InstanceHandle, WorldSpaceInstanceTransform, IterationContext);
					if (!bContinue)
					{
						break;
					}
				}

				++InstanceIndex;
			}

			if (!bContinue)
			{
				break;
			}
		}
	}

	return bContinue;
}

template <typename TBoundsType>
bool AInstancedActorsManager::ForEachInstance(const TBoundsType& QueryBounds, FInstanceOperationFunc Operation) const
{
	FScopedInstancedActorsIterationContext IterationContext;

	return ForEachInstance(QueryBounds, Operation, IterationContext);
}

template <typename TBoundsType>
bool AInstancedActorsManager::ForEachInstance(const TBoundsType& QueryBounds, FInstanceOperationFunc Operation, FInstancedActorsIterationContext& IterationContext, TOptional<FInstancedActorDataPredicateFunc> InstancedActorDataPredicate) const
{
	return ForEachInstance([&QueryBounds, &Operation](const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext)
		{
			if (UE::InstancedActors::PassesBoundsTest(QueryBounds, UE::InstancedActors::EBoundsTestType::Intersect, InstanceHandle, InstanceTransform))
			{
				return Operation(InstanceHandle, InstanceTransform, IterationContext);
			}
			return true;
		},
		IterationContext, InstancedActorDataPredicate);
}

// Instantiate FBox and FSphere implementations
template bool AInstancedActorsManager::ForEachInstance<FBox>(const FBox& QueryBounds, AInstancedActorsManager::FInstanceOperationFunc Operation) const;
template bool AInstancedActorsManager::ForEachInstance<FSphere>(const FSphere& QueryBounds, AInstancedActorsManager::FInstanceOperationFunc Operation) const;

template<>
UE::InstancedActors::EInsideBoundsTestResult AInstancedActorsManager::IsInstanceInsideBounds<FBox>(const FBox& QueryBounds
	, const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform)
{
	if (QueryBounds.IsInside(InstanceTransform.GetLocation()))
	{
		return UE::InstancedActors::EInsideBoundsTestResult::OverlapLocation;
	}

	// More expensive bounds test.
	const FBox InstancedActorBounds = InstanceHandle.InstancedActorData->CachedLocalBounds.TransformBy(InstanceTransform);
	if (QueryBounds.Intersect(InstancedActorBounds))
	{
		return UE::InstancedActors::EInsideBoundsTestResult::OverlapBounds;
	}

	return UE::InstancedActors::EInsideBoundsTestResult::NotInside;
}

template<>
UE::InstancedActors::EInsideBoundsTestResult AInstancedActorsManager::IsInstanceInsideBounds<FSphere>(const FSphere& QueryBounds, const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform)
{
	if (QueryBounds.IsInside(InstanceTransform.GetLocation()))
	{
		return UE::InstancedActors::EInsideBoundsTestResult::OverlapLocation;
	}

	// More expensive bounds test.
	const FBox& InstancedActorBounds = InstanceHandle.InstancedActorData->CachedLocalBounds;
	const FSphere TransformedSphere = QueryBounds.TransformBy(InstanceTransform.Inverse());
	if (FMath::SphereAABBIntersection(TransformedSphere, InstancedActorBounds))
	{
		return UE::InstancedActors::EInsideBoundsTestResult::OverlapBounds;
	}

	return UE::InstancedActors::EInsideBoundsTestResult::NotInside;
}

bool AInstancedActorsManager::HasInstancesOfClass(const FBox& InQueryBounds, TSubclassOf<AActor> ActorClass
	, const bool bTestActorsIfSpawned, const EInstancedActorsBulkLODMask AllowedLODs) const
{
	using UE::InstancedActors::EInsideBoundsTestResult;

	ensure(ActorClass);
	check(InstancedActorSubsystem);

	FScopedInstancedActorsIterationContext IterationContext;
	bool bHasInstance = false;

	auto InstancedActorDataMask = TOptional<AInstancedActorsManager::FInstancedActorDataPredicateFunc>([=](const UInstancedActorsData& InstancedActorData)
		{
			return ((int(AllowedLODs) & (1 << int(InstancedActorData.GetBulkLOD()))) != 0)
				&& InstancedActorData.ActorClass->IsChildOf(ActorClass);
		});

	struct FQueryBounds
	{
		FBox QueryBounds;
		FVector Center;
		FVector Extent;
		FCollisionShape CollistionShape;

		FQueryBounds(const FBox& InQueryBounds)
			: QueryBounds(InQueryBounds)
		{
			InQueryBounds.GetCenterAndExtents(Center, Extent);
			CollistionShape = FCollisionShape::MakeBox(Extent);
		}
	};
	FQueryBounds CachedQueryBounds(InQueryBounds);

	ForEachInstance([Manager = this, CachedQueryBounds, ActorClass, &bHasInstance, InstancedActorSubsystem=InstancedActorSubsystem, bTestActorsIfSpawned](const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext)
		{
			const EInsideBoundsTestResult OverlapResult = Manager->IsInstanceInsideBounds(CachedQueryBounds.QueryBounds, InstanceHandle, InstanceTransform);
			
			// the only case we care about, representation bounds overlap without overlapping the actual instance location
			if (OverlapResult == EInsideBoundsTestResult::OverlapBounds && bTestActorsIfSpawned)
			{
				// we want to check if the given entity has an actor representation, and if so then test the actor itself
				UInstancedActorsData& OwningInstanceData = InstanceHandle.GetInstanceActorDataChecked();
				if (AActor* Actor = Manager->GetActorForInstance(OwningInstanceData, InstanceHandle.GetIndex()))
				{
					bHasInstance = false;

					TArray<UPrimitiveComponent*> OutComponents;
					Actor->GetComponents(UPrimitiveComponent::StaticClass(), OutComponents, /*bIncludeFromChildActors=*/true);
					for (UPrimitiveComponent* Component : OutComponents)
					{
						if (Component->OverlapComponent(CachedQueryBounds.Center, FQuat::Identity, CachedQueryBounds.CollistionShape))
						{
							bHasInstance = true;
							break;
						}
					}
				}
				else
				{
					// if the bounds overlap but there's no actor we need to say it's a hit, since we cannot rule that out.
					bHasInstance = true;
				}
			}
			else 
			{
				bHasInstance = (OverlapResult != EInsideBoundsTestResult::NotInside);
			}
				
			UE_IFVLOG
			(
				if (bHasInstance || OverlapResult == EInsideBoundsTestResult::OverlapBounds)
				{
					const FBox InstancedActorBounds = InstanceHandle.InstancedActorData->CachedLocalBounds.TransformBy(InstanceTransform);
					UE_VLOG_BOX(Manager, LogInstancedActors, Log, InstancedActorBounds, bHasInstance ? FColor::Red : FColor::Green
						, TEXT("Instance of class %s"), *GetNameSafe(InstanceHandle.InstancedActorData->ActorClass));
				}
			);

			const bool bContinue = !bHasInstance;
			return bContinue;
		}
		, IterationContext, InstancedActorDataMask);

	return bHasInstance;
}

void AInstancedActorsManager::AuditInstances(FOutputDevice& Ar, bool bDebugDraw, float DebugDrawDuration) const
{
	UWorld* World = GetWorld();
	const bool bInGameWorld = World->IsGameWorld();

	FRandomStream RandomStream(GetUniqueID());

#if WITH_EDITOR
	Ar.Logf(TEXT("%s (%s)"), *GetActorLabel(), *GetName());
#else
	Ar.Logf(TEXT("%s"), *GetName());
#endif

	Ar.Logf(TEXT("\tLevel: %s"), *GetLevel()->GetPathName());
	Ar.Logf(TEXT("\tLevel Transform: %s"), *GetLevelTransform().ToString());
	Ar.Logf(TEXT("\tLocation: %s"), *GetActorLocation().ToString());
	Ar.Logf(TEXT("\tNum Instances: %d"), GetNumValidInstances());
#if WITH_EDITOR
	Ar.Logf(TEXT("\tRuntime grid: %s"), *GetRuntimeGrid().ToString());
#endif

#if UE_ENABLE_DEBUG_DRAWING
	if (bDebugDraw)
	{
		// Draw manager center and extent
		FVector BoundsCenter;
		FVector BoundsExtent;
		GetActorBounds(/*bOnlyCollidingComponents*/ false, BoundsCenter, BoundsExtent);
		DrawDebugBox(World, BoundsCenter, BoundsExtent, FColor::White, /*bPersistentLines*/ DebugDrawDuration == -1.0f, DebugDrawDuration);
		DrawDebugPoint(World, GetActorLocation(), 100.0f, FColor::White, /*bPersistentLines*/ DebugDrawDuration == -1.0f, DebugDrawDuration);

#if WITH_EDITOR
		// Draw streaming bounds
		FBox RuntimeBounds;
		FBox EditorBounds;
		GetStreamingBounds(RuntimeBounds, EditorBounds);
		DrawDebugBox(World, RuntimeBounds.GetCenter(), RuntimeBounds.GetExtent(), FColor::Orange, /*bPersistentLines*/ DebugDrawDuration == -1.0f, DebugDrawDuration);
#endif // WITH_EDITOR
	}
#endif // UE_ENABLE_DEBUG_DRAWING

	const FVector ManagerLocation = GetActorLocation();
	const FTransform& ManagerTransform = GetActorTransform();
	const bool bApplyManagerTranslationOnly = (GetActorQuat().IsIdentity() && GetActorScale().Equals(FVector::OneVector));
	for (UInstancedActorsData* InstanceData : PerActorClassInstanceData)
	{
		Ar.Logf(TEXT("\t%s"), *InstanceData->GetDebugName(/*bCompact*/ true));
		if (bInGameWorld)
		{
			const FInstancedActorsSettings* Settings = InstanceData->GetSettingsPtr<const FInstancedActorsSettings>();
			Ar.Logf(TEXT("\t\tSettings: %s"), Settings ? *Settings->DebugToString() : TEXT("?"));
		}
		Ar.Logf(InstanceData->NumValidInstances > 0 ? ELogVerbosity::Log : ELogVerbosity::Warning, TEXT("\t\tNum Instances: %u"), InstanceData->NumValidInstances);
		if (InstanceData->GetNumFreeInstances() > 0)
		{
			Ar.Logf(ELogVerbosity::Warning, TEXT("\t\tNum Free Instances: %d - these incur extra runtime cost! Consider running IA.CompactInstances console command"), InstanceData->GetNumFreeInstances());
		}
		if (!InstanceData->Bounds.IsValid)
		{
			Ar.Logf(ELogVerbosity::Error, TEXT("\t\t%s has invalid bounds (%s)"), *InstanceData->GetDebugName(/*bCompact*/ false), *InstanceData->Bounds.ToString());
		}
		else
		{
			Ar.Logf(TEXT("\t\tLocal Bounds: %s"), *InstanceData->Bounds.ToString());
		}

		for (uint8 VisualizationIndex = 0; VisualizationIndex < InstanceData->InstanceVisualizations.Num(); ++VisualizationIndex)
		{
			if (const FInstancedActorsVisualizationInfo* Visualization = InstanceData->GetVisualization(VisualizationIndex))
			{
				FString VisualizationString;
				FInstancedActorsVisualizationInfo::StaticStruct()->ExportText(VisualizationString, Visualization, /*Defaults*/ nullptr, /*OwnerObject*/ InstanceData, PPF_None, /*ExportRootScope*/ nullptr);
				Ar.Logf(TEXT("\t\tVisualization %u: %s"), VisualizationIndex, *VisualizationString);
			}
		}

		const int32 NumDeltas = InstanceData->InstanceDeltas.GetInstanceDeltas().Num();
		if (NumDeltas > 0)
		{
			Ar.Logf(TEXT("\t\tNum Deltas: %d (Destroyed: %u, Lifecycle Phases: %u)"), NumDeltas, InstanceData->InstanceDeltas.GetNumDestroyedInstanceDeltas(), InstanceData->InstanceDeltas.GetNumLifecyclePhaseDeltas());
		}

#if UE_ENABLE_DEBUG_DRAWING
		if (bDebugDraw)
		{
			FColor InstanceTypeColor = InstanceData->ActorClass ? FColor::MakeRandomSeededColor(InstanceData->ActorClass->GetUniqueID()) : FColor::Black;

			// Draw random colored point for each instance with 20cm jitter to show overlapped instances
			for (const FTransform& InstanceTransform : InstanceData->InstanceTransforms)
			{
				if (UE::InstancedActors::IsValidInstanceTransform(InstanceTransform))
				{
					FVector WorldLocation = bApplyManagerTranslationOnly ? ManagerLocation + InstanceTransform.GetLocation() : ManagerTransform.TransformPosition(InstanceTransform.GetLocation());
					FVector JitteredWorldLocation = WorldLocation + RandomStream.VRand() * 20.0f;
					FColor InstanceColor = FLinearColor(RandomStream.GetFraction(), RandomStream.GetFraction(), RandomStream.GetFraction()).ToFColor(true);
					DrawDebugLine(World, WorldLocation, JitteredWorldLocation, InstanceColor, /*bPersistentLines*/ DebugDrawDuration == -1.0f, DebugDrawDuration);
					DrawDebugPoint(World, JitteredWorldLocation, 50.0f, InstanceTypeColor, /*bPersistentLines*/ DebugDrawDuration == -1.0f, DebugDrawDuration);
				}
			}
		}
#endif
	}
}

void AInstancedActorsManager::CompactInstances(FOutputDevice& Ar)
{
	Modify();

	for (UInstancedActorsData* InstanceData : PerActorClassInstanceData)
	{
		const int32 NumFreeInstances = InstanceData->GetNumFreeInstances();
		if (NumFreeInstances > 0)
		{
			Ar.Logf(TEXT("Compacting %s by fully removing %d free indices"), *InstanceData->GetDebugName(), InstanceData->GetNumFreeInstances());

			int32 NumInstancesRemoved = 0;
			for (int32 Index = 0; Index < InstanceData->InstanceTransforms.Num();)
			{
				if (!UE::InstancedActors::IsValidInstanceTransform(InstanceData->InstanceTransforms[Index]))
				{
					InstanceData->InstanceTransforms.RemoveAtSwap(Index);
					++NumInstancesRemoved;
				}
				else
				{
					++Index;
				}
			}
			check(NumInstancesRemoved == NumFreeInstances);

			InstanceData->NumValidInstances = InstanceData->InstanceTransforms.Num();

#if WITH_EDITORONLY_DATA
			// Modify the ISMCs here since the components might have been serialized before this change.
			for (TObjectPtr<UInstancedStaticMeshComponent> ISMComponent : InstanceData->EditorPreviewISMComponents)
			{
				ISMComponent->ClearInstances();
				ISMComponent->AddInstances(InstanceData->InstanceTransforms, false, false, false);
			}
#endif
		}
	}
}

int32 AInstancedActorsManager::GetNumValidInstances() const
{
	int32 InstanceCountTotal = 0;
	for (UInstancedActorsData* InstanceData : PerActorClassInstanceData)
	{
		InstanceCountTotal += InstanceData->NumValidInstances;
	}

	return InstanceCountTotal;
}

bool AInstancedActorsManager::HasAnyValidInstances() const
{
	for (UInstancedActorsData* InstanceData : PerActorClassInstanceData)
	{
		if (InstanceData->NumValidInstances > 0)
		{
			return true;
		}
	}

	return false;
}

bool AInstancedActorsManager::IsValidInstance(const FInstancedActorsInstanceHandle& InstanceHandle) const
{
	return InstanceHandle.IsValid() && InstanceHandle.GetManager() == this && InstanceHandle.InstancedActorData->IsValidInstance(InstanceHandle);
}

void AInstancedActorsManager::AddModifierVolume(UInstancedActorsModifierVolumeComponent& ModifierVolume)
{
	check(!ModifierVolumes.Contains(&ModifierVolume));

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	UE_LOGF(LogInstancedActors, Verbose, "%ls @ %ls adding modifier volume %ls @ %ls", *GetPathName(), *GetActorLocation().ToString(), *ModifierVolume.GetReadableName(), *ModifierVolume.GetComponentLocation().ToString());
#endif

	ModifierVolumes.Add(&ModifierVolume);
	PendingModifierVolumes.Add(true);
	PendingModifierVolumeModifiers.Add(TBitArray<>(true, ModifierVolume.Modifiers.Num()));

	ModifierVolume.OnAddedToManager(*this);

	// If this volume is being added latently after InitializeModifyAndSpawnEntities, it will have missed
	// TryRunPendingModifiers so re-run this now. Prior to InitializeModifyAndSpawnEntities we simply
	// register the volume as pending for later processing in InitializeModifyAndSpawnEntities.
	if (HasSpawnedEntities())
	{
		TryRunPendingModifiers();
	}
}

void AInstancedActorsManager::RemoveModifierVolume(UInstancedActorsModifierVolumeComponent& ModifierVolume)
{
	check(ModifierVolumes.Contains(&ModifierVolume));

#if WITH_INSTANCEDACTORS_DEBUG
	UE_LOGF(LogInstancedActors, Verbose, "%ls @ %ls removing modifier volume %ls @ %ls", *GetPathName(), *GetActorLocation().ToString(), *ModifierVolume.GetReadableName(), *ModifierVolume.GetComponentLocation().ToString());
#endif

	int32 ModifierVolumeIndex = ModifierVolumes.IndexOfByKey(&ModifierVolume);
	ModifierVolumes.RemoveAtSwap(ModifierVolumeIndex);
	PendingModifierVolumes.RemoveAtSwap(ModifierVolumeIndex);
	PendingModifierVolumeModifiers.RemoveAtSwap(ModifierVolumeIndex);

	ModifierVolume.OnRemovedFromManager(*this);
}

void AInstancedActorsManager::RemoveAllModifierVolumes()
{
	for (TWeakObjectPtr<UInstancedActorsModifierVolumeComponent> WeakVolumeComponent : ModifierVolumes)
	{
		if (UInstancedActorsModifierVolumeComponent* VolumeComponent = WeakVolumeComponent.Get())
		{
			VolumeComponent->OnRemovedFromManager(*this);
		}
	}
	
	ModifierVolumes.Reset();
	PendingModifierVolumes.Reset();
	PendingModifierVolumeModifiers.Reset();
}

void AInstancedActorsManager::TryRunPendingModifiers()
{
	for (TBitArray<>::FIterator PendingModifierVolumeIt(PendingModifierVolumes); PendingModifierVolumeIt; ++PendingModifierVolumeIt)
	{
		if (PendingModifierVolumeIt)
		{
			const int32 ModifierVolumeIndex = PendingModifierVolumeIt.GetIndex();
			check(ModifierVolumes.IsValidIndex(ModifierVolumeIndex));
			UInstancedActorsModifierVolumeComponent* PendingModifierVolume = ModifierVolumes[ModifierVolumeIndex].Get();
			if (ensure(IsValid(PendingModifierVolume)))
			{
				check(PendingModifierVolumeModifiers.IsValidIndex(ModifierVolumeIndex));
				TBitArray<>& PendingModifiers = PendingModifierVolumeModifiers[ModifierVolumeIndex];

				const bool bRanAllPendingModifiers = PendingModifierVolume->TryRunPendingModifiers(*this, PendingModifiers);

				// Clear whole volume dirty state (false) if all modifiers ran, set dirty (true) otherwise
				PendingModifierVolumeIt.GetValue() = !bRanAllPendingModifiers;
			}
		}
	}
}

FBox AInstancedActorsManager::CalculateBounds(TSubclassOf<AActor> ActorClass)
{
	if (const UStaticMeshComponent* SourceStaticMeshComponent = AActor::GetActorClassDefaultComponent<UStaticMeshComponent>(ActorClass))
	{
		if (UStaticMesh* StaticMesh = SourceStaticMeshComponent->GetStaticMesh())
		{
			return StaticMesh->GetBoundingBox();
		}
	}

	// Return 0-sized valid bounds for non-mesh assets so we can still instance them by
	// location.
	return FBox(FVector::ZeroVector, FVector::ZeroVector);
}

void AInstancedActorsManager::UpdateInstanceStats(int32 InstanceCount, EInstancedActorsBulkLOD LODMode, bool Increment)
{
#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
	InstanceCount = Increment ? InstanceCount : -InstanceCount;

	// dec stats on current render mode
	switch (LODMode)
	{
	case EInstancedActorsBulkLOD::Detailed:
	{
		INC_DWORD_STAT_BY(STAT_DetailedInstanceCount, InstanceCount);
		break;
	}
	case EInstancedActorsBulkLOD::Medium:
	{
		INC_DWORD_STAT_BY(STAT_MediumInstanceCount, InstanceCount);
		break;
	}
	case EInstancedActorsBulkLOD::Low:
	{
		INC_DWORD_STAT_BY(STAT_LowLODInstanceCount, InstanceCount);
		break;
	}
	case EInstancedActorsBulkLOD::Off:
	{
		INC_DWORD_STAT_BY(STAT_OffInstanceCount, InstanceCount);
		break;
	}
	}
#endif // !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
}

void AInstancedActorsManager::RequestPersistentDataSave()
{
	if (!UE::InstancedActors::CVars::bEnablePersistence)
	{
		return;
	}

	RequestActorSave(this);
}

void AInstancedActorsManager::OnPersistentDataRestored()
{
	if (!UE::InstancedActors::CVars::bEnablePersistence)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE(AInstancedActorsManager::OnPersistentDataRestored);

	for (TObjectPtr<UInstancedActorsData> InstanceData : PerActorClassInstanceData)
	{
		check(InstanceData);

		InstanceData->OnPersistentDataRestored();
	}
}

FBox AInstancedActorsManager::CalculateLocalInstanceBounds() const
{
	FBox Bounds(ForceInit);
	for (TObjectPtr<UInstancedActorsData> InstanceData : PerActorClassInstanceData)
	{
		Bounds += InstanceData->Bounds;
	}
	return Bounds;
}

void AInstancedActorsManager::RegisterInstanceDatasComponents(const UInstancedActorsData& InstanceData, TConstArrayView<TObjectPtr<UInstancedStaticMeshComponent>> Components)
{
	for (TObjectPtr<UInstancedStaticMeshComponent> Component : Components)
	{
		ISMComponentToInstanceDataMap.Add(Component, InstanceData.GetInstanceDataID());
	}
}

void AInstancedActorsManager::UnregisterInstanceDatasComponent(UInstancedStaticMeshComponent& Component)
{
	ensureMsgf(ISMComponentToInstanceDataMap.Remove(&Component) > 0, TEXT("Trying to unregister %s but it's cannot be found in ISMComponentToInstanceDataMap"), *Component.GetPathName());
}

void AInstancedActorsManager::CreateISMComponents(const FInstancedActorsVisualizationDesc& VisualizationDesc, FConstSharedStruct SharedSettings
	, TArray<TObjectPtr<UInstancedStaticMeshComponent>>& OutComponents, const bool bEditorPreviewISMCs)
{
	const bool bEnableInstanceCollision = (GetNetMode() == NM_Client) 
		? UE::InstancedActors::CVars::bInstanceCollisionsOnClient
		: UE::InstancedActors::CVars::bInstanceCollisionsOnServer;

	const FInstancedActorsSettings* Settings = SharedSettings.GetPtr<const FInstancedActorsSettings>();

	for (const FISMComponentDescriptor& ISMCDescriptor : VisualizationDesc.ISMComponentDescriptors)
	{
		UInstancedStaticMeshComponent* ISMComponent = NewObject<UInstancedStaticMeshComponent>(this);
		// the following two checks are meant to catch unexpected reuse of existing UInstancedStaticMeshComponent (which can happen if the generated name is being used already).
		checkf(ISMComponent->GetOwner() == this, TEXT("Newly created ISM component seems to be a reused one - it has a different owner,\nexpected: %s,\nactual: %s"), *GetNameSafe(ISMComponent->GetOwner()), *GetName());
		checkf(ISMComponent->GetAttachParent() == nullptr, TEXT("Newly created ISM component already has a non-null attach parent: %s"), *ISMComponent->GetAttachParent()->GetName());

		ISMCDescriptor.InitComponent(ISMComponent);

		// IA enforced overrides
		ISMComponent->SetMobility(EComponentMobility::Stationary);
		ISMComponent->SetupAttachment(RootComponent);

		// @todo Add support for non-replay NM_Standalone where we should use bInstanceCollisionsOnServer
		ISMComponent->bDisableCollision = !bEnableInstanceCollision;

		// Note: The base profile doesn't use Physics, only Query, but some use cases might rely on physics
		// so we need the IAM's to ensure that is turned on.
		// Note: Setting both bDisableCollision and ECollisionEnabled::NoCollision when disabling collision, to
		// 		 avoid confusion of conflicting values.
		ISMComponent->SetCollisionEnabled(bEnableInstanceCollision ? ECollisionEnabled::QueryAndPhysics : ECollisionEnabled::NoCollision);

		// Use conservative bounds to decrease bounds calculation cost when we have high instance counts.
		ISMComponent->SetUseConservativeBounds(true);
		// Compute fast local bounds works with the above setting so that the conservative bounds are used wheen calculating local bounds.
		ISMComponent->bComputeFastLocalBounds = true;

		static FName NAME_StatsCategoryLightWeightInstance("LightWeightInstance");
		ISMComponent->SetMeshDrawCommandStatsCategory(NAME_StatsCategoryLightWeightInstance);

		// Set the default world position offset disable distance
		ISMComponent->WorldPositionOffsetDisableDistance = UE::InstancedActors::CVars::WorldPositionOffsetDisableDistance * UE::InstancedActors::CVars::WorldPositionOffsetDisableDistanceScale;

		// Settings overrides
		ensureMsgf(Settings != nullptr || bEditorPreviewISMCs, TEXT("CreateISMComponents called for runtime component creation without FInstancedActorsSettings"));
		if (Settings)
		{
			if (Settings->bOverride_bCanEverAffectNavigation)
			{
				ISMComponent->SetCanEverAffectNavigation(Settings->bCanEverAffectNavigation);
			}
			else
			{
				// By default IAMs should affect navigation.
				ISMComponent->SetCanEverAffectNavigation(true);
			}

			// Optional settings
			if (Settings->bOverride_bInstancesCastShadows)
			{
				ISMComponent->SetCastShadow(Settings->bInstancesCastShadows);
			}

			if (Settings->bOverride_WorldPositionOffsetDisableDistance)
			{
				ISMComponent->WorldPositionOffsetDisableDistance = Settings->WorldPositionOffsetDisableDistance * UE::InstancedActors::CVars::WorldPositionOffsetDisableDistanceScale;
			}

			if (Settings->bOverride_AffectDistanceFieldLighting)
			{
				ISMComponent->bAffectDistanceFieldLighting = Settings->GetAffectDistanceFieldLighting();
			}
		}

		if (UE::InstancedActors::CVars::bOverrideCastFarShadow)
		{
			ISMComponent->bCastFarShadow = ISMComponent->bAffectDistanceFieldLighting;
		}

		// Editor preview overrides
		if (bEditorPreviewISMCs)
		{
			SetUpEditorPreviewISMComponent(ISMComponent);
		}

		ISMComponent->PrecachePSOs();

		ISMComponent->RegisterComponent();

		OutComponents.Add(ISMComponent);
	}
}

void AInstancedActorsManager::SetUpEditorPreviewISMComponent(TNotNull<UInstancedStaticMeshComponent*> ISMComponent)
{
#if WITH_EDITORONLY_DATA
	ISMComponent->SetIsVisualizationComponent(false);
#endif //WITH_EDITORONLY_DATA
	ISMComponent->SetCollisionEnabled(ECollisionEnabled::QueryOnly);
	ISMComponent->bSelectable = true;
	ISMComponent->bHasPerInstanceHitProxies = true;
	ISMComponent->bIsEditorOnly = true;
}

//-----------------------------------------------------------------------------
// AInstancedActorsManager IActorInstanceManagerInterface overrides
//-----------------------------------------------------------------------------
int32 AInstancedActorsManager::ConvertCollisionIndexToInstanceIndex(int32 InIndex, const UPrimitiveComponent* RelevantComponent) const
{
	QUICK_SCOPE_CYCLE_COUNTER(IA_ConvertCollisionIndexToInstanceIndex);

#if WITH_EDITOR
	const UWorld* World = GetWorld();
	if (World != nullptr && !World->IsGameWorld())
	{
		// In Editor non-game worlds the manager relies only EditorPreviewISMComponents
		// and InstancedActorData doesn't have InstanceVisualizations created.
		return INDEX_NONE;
	}
#endif // WITH_EDITOR

	if (UE::InstancedActors::CVars::bInstantHydrationViaPhysicsQueriesEnabled == false || InIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}
	checkf(RelevantComponent, TEXT("RelevantComponent is required to not be null for Instanced Actors"));

	const UInstancedStaticMeshComponent* AsISMComponent = Cast<UInstancedStaticMeshComponent>(RelevantComponent);
	if (AsISMComponent == nullptr)
	{
		return INDEX_NONE;
	}

	if (const int32* InstanceDataID = ISMComponentToInstanceDataMap.Find(AsISMComponent))
	{
		check(PerActorClassInstanceData.IsValidIndex(*InstanceDataID));
		check(PerActorClassInstanceData[*InstanceDataID]);
		const int32 EntityIndex = PerActorClassInstanceData[*InstanceDataID]->GetEntityIndexFromCollisionIndex(*AsISMComponent, InIndex);
		return EntityIndex != INDEX_NONE
			? FInstancedActorsInstanceIndex::BuildCompositeIndex(*InstanceDataID, EntityIndex)
			: INDEX_NONE;
	}

	return INDEX_NONE;
}

AActor* AInstancedActorsManager::FindActorInternal(const FActorInstanceHandle& Handle, FMassEntityView& OutEntityView, const bool bEnsureOnMissingInstanceDataOrMassEntity) const
{
#if WITH_EDITOR
	const UWorld* World = GetWorld();
	if (World != nullptr && !World->IsGameWorld() && Handle.IsValid() == false)
	{
		// In Editor non-game worlds the manager only creates preview ISM components which
		// are not fully setup with Mass so we simply return this manager as the instance's actor.
		return const_cast<AInstancedActorsManager*>(this);
	}
#endif // WITH_EDITOR

	if (UE::InstancedActors::CVars::bInstantHydrationViaPhysicsQueriesEnabled == false || Handle.IsValid() == false)
	{
		return nullptr;
	}

	if (AActor* CachedActor = Handle.GetCachedActor())
	{
		return CachedActor;
	}

	if (IsNetMode(NM_Client))
	{
		// On clients, where we don't spawn actors directly, we simply return this manager as the
		// instance's actor.
		return const_cast<AInstancedActorsManager*>(this);
	}

	const int32 CompositeIndex = Handle.GetInstanceIndex();
	const int32 InstancedActorDataIndex = FInstancedActorsInstanceIndex::ExtractInstanceDataID(CompositeIndex);
	const FInstancedActorsInstanceIndex InternalInstanceIndex = FInstancedActorsInstanceIndex(FInstancedActorsInstanceIndex::ExtractInternalInstanceIndex(CompositeIndex));

	if (PerActorClassInstanceData.IsValidIndex(InstancedActorDataIndex))
	{
		const UInstancedActorsData* InstanceData = PerActorClassInstanceData[InstancedActorDataIndex];
		check(IsValid(InstanceData));

		if (InstanceData->CanHydrate())
		{
			const FMassEntityHandle EntityHandle = InstanceData->GetEntityHandleForIndex(InternalInstanceIndex);
			// note that it's possible that InternalInstanceIndex points at a no-longer-valid entity, if
			// the entity got already destroyed due to other responses to the physics query that
			// generated the given FActorInstanceHandle handle. This can easily be reproduced if actors
			// get destroyed as a result of being hit.
			if (EntityHandle.IsValid())
			{
				const FMassEntityManager& EntityManager = GetMassEntityManagerChecked();
				OutEntityView = FMassEntityView::TryMakeView(EntityManager, EntityHandle);
				if (OutEntityView.IsValid())
				{
					if (FMassActorFragment* ActorFragment = OutEntityView.GetFragmentDataPtr<FMassActorFragment>())
					{
						return ActorFragment->GetMutable();
					}
				}
#if DO_ENSURE
				else
				{
					ensureMsgf(!bEnsureOnMissingInstanceDataOrMassEntity, TEXT("Unable to fetch a valid FMassEntityView for entity handle [%s]"), *EntityHandle.DebugGetDescription());
				}
#endif // DO_ENSURE
			}
		}
	}
#if DO_ENSURE
	else
	{
		ensureMsgf(!bEnsureOnMissingInstanceDataOrMassEntity, TEXT("Unable to fetch a valid UInstancedActorsData for index [%d]"), InstancedActorDataIndex);
	}
#endif // DO_ENSURE

	return nullptr;
}

AActor* AInstancedActorsManager::GetActorForInstance(const UInstancedActorsData& InstanceData, const int32 InstancedActorIndex) const
{
	const FMassEntityHandle EntityHandle = InstanceData.GetEntityHandleForIndex(InstancedActorIndex);
	// note that it's possible that InstancedActorIndex points at a no-longer-valid entity
	if (EntityHandle.IsValid())
	{
		const FMassEntityManager& EntityManager = GetMassEntityManagerChecked();
		if (FMassActorFragment* ActorFragment = EntityManager.GetFragmentDataPtr<FMassActorFragment>(EntityHandle))
		{
			return ActorFragment->GetMutable();
		}
	}
	return nullptr;
}

AActor* AInstancedActorsManager::FindActor(const FActorInstanceHandle& Handle)
{
	FMassEntityView EntityView;
	return FindActorInternal(Handle, EntityView, /*bEnsureOnMissingInstanceDataOrMassEntity=*/false);
}

AActor* AInstancedActorsManager::FindOrCreateActor(const FActorInstanceHandle& Handle)
{
	FMassEntityView EntityView;
	AActor* Actor = FindActorInternal(Handle, EntityView, /*bEnsureOnMissingInstanceDataOrMassEntity=*/true);

	// Create missing actor from Mass if we have a valid EntityView
	if (Actor == nullptr && EntityView.IsValid())
	{
#if WITH_INSTANCEDACTORS_DEBUG
		{
			const int32 CompositeIndex = Handle.GetInstanceIndex();
			const int32 InstancedActorDataIndex = FInstancedActorsInstanceIndex::ExtractInstanceDataID(CompositeIndex);
			const FInstancedActorsInstanceIndex InternalInstanceIndex = FInstancedActorsInstanceIndex(FInstancedActorsInstanceIndex::ExtractInternalInstanceIndex(CompositeIndex));
			const UInstancedActorsData* InstanceData = PerActorClassInstanceData.IsValidIndex(InstancedActorDataIndex) ? PerActorClassInstanceData[InstancedActorDataIndex] : nullptr;
			ensureMsgf(InstanceData && InstanceData->CanHydrate()
				, TEXT("We're about to spawn an actor while the relevant IAD %s has been configured to not hydrate its instances")
				, InstanceData ? *InstanceData->GetDebugName() : TEXT("NONE"));
		}
#endif // WITH_INSTANCEDACTORS_DEBUG

		UMassRepresentationSubsystem* RepresentationSubsystem = EntityView.GetSharedFragmentData<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
		const FMassRepresentationParameters& RepresentationParams = EntityView.GetConstSharedFragmentData<FMassRepresentationParameters>();
		UInstancedActorsRepresentationActorManagement* RepresentationActorManagement = Cast<UInstancedActorsRepresentationActorManagement>(RepresentationParams.CachedRepresentationActorManagement);

		Actor = RepresentationActorManagement->FindOrInstantlySpawnActor(*RepresentationSubsystem, GetMassEntityManagerChecked(), EntityView);

#if DO_ENSURE
		if (Actor == nullptr)
		{
			const int32 CompositeIndex = Handle.GetInstanceIndex();
			const int32 InstancedActorDataIndex = FInstancedActorsInstanceIndex::ExtractInstanceDataID(CompositeIndex);
			const FInstancedActorsInstanceIndex InternalInstanceIndex = FInstancedActorsInstanceIndex(FInstancedActorsInstanceIndex::ExtractInternalInstanceIndex(CompositeIndex));
			const UInstancedActorsData* InstanceData = PerActorClassInstanceData[InstancedActorDataIndex];
			ensureMsgf(Actor, TEXT("Failed spawning actor for %s actor instance handle %d (Instance Index: %d)"), *InstanceData->GetDebugName(), CompositeIndex, InternalInstanceIndex.GetIndex());
		}
#endif // DO_ENSURE
	}

	// If we failed to spawn an actor, return ourselves as the hit / overlapped actor for InstanceHandle.
	// @todo Validate with Mieszko if this comment is wrong or if the return value should be 'this' if Actor is still nullptr
	return Actor;
}

UClass* AInstancedActorsManager::GetRepresentedClass(const int32 InstanceIndex) const
{
	if (UE::InstancedActors::CVars::bInstantHydrationViaPhysicsQueriesEnabled == false || InstanceIndex == INDEX_NONE)
	{
		return nullptr;
	}

	// On clients, where we don't spawn actors directly, we simply return this manager as the
	// FActorInstanceHandle's actor.
	if (IsNetMode(NM_Client))
	{
		return GetClass();
	}

	const int32 InstanceDataID = FInstancedActorsInstanceIndex::ExtractInstanceDataID(InstanceIndex);
	return ensure(PerActorClassInstanceData.IsValidIndex(InstanceDataID))
		? PerActorClassInstanceData[InstanceDataID]->ActorClass
		: nullptr;
}

ULevel* AInstancedActorsManager::GetLevelForInstance(const int32 InstanceIndex) const
{
	return GetLevel();
}

FTransform AInstancedActorsManager::GetTransform(const FActorInstanceHandle& Handle) const
{
	if (AActor* CachedActor = Handle.GetCachedActor())
	{
		return CachedActor->GetTransform();
	}

	return FTransform();
}

// END AInstancedActorsManager IActorInstanceManagerInterface overrides

//~ Begin ISMInstanceManager Overrides

FText AInstancedActorsManager::GetSMInstanceDisplayName(const FSMInstanceId& InstanceId) const
{
#if WITH_EDITOR
	FText TypeName = InstanceId.ISMComponent ? FText::FromString(InstanceId.ISMComponent->GetName()) : FText();
	FText OwnerDisplayName = FText::FromString(GetActorLabel());

	const FInstancedActorsInstanceHandle InstanceHandle = ActorInstanceHandleFromFSMInstanceId(InstanceId);
	if (ensure(InstanceHandle.IsValid()))
	{
		const UInstancedActorsData& InstanceData = InstanceHandle.GetInstanceActorDataChecked();
		OwnerDisplayName = FText::FromString(InstanceData.GetManagerChecked().GetActorLabel());

		FName TypeFName = InstanceData.ActorClass->GetFName();
		if (InstanceData.ActorClass->ClassGeneratedBy)
		{
			TypeFName = InstanceData.ActorClass->ClassGeneratedBy->GetFName();
		}
		TypeName = FText::FromName(TypeFName);
	}

	return FText::Format(NSLOCTEXT("InstancedActorsManager", "DisplayNameFmt", "{0}.{1}[{2}]"), OwnerDisplayName, TypeName, InstanceId.InstanceIndex);
#else
	return FText();
#endif
}

FText AInstancedActorsManager::GetSMInstanceTooltip(const FSMInstanceId& InstanceId) const
{
	// Not sure what would be better as a tool tip. This is useful at least since the names can get too long for the details panel.
	return GetSMInstanceDisplayName(InstanceId);
}

bool AInstancedActorsManager::CanEditSMInstance(const FSMInstanceId& InstanceId) const
{
#if WITH_EDITORONLY_DATA
	return !bLockInstanceLocation;
#else
	return false;
#endif
}

bool AInstancedActorsManager::CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType InWorldType) const
{
#if WITH_EDITORONLY_DATA
	return !bLockInstanceLocation;
#else
	return false;
#endif
}

bool AInstancedActorsManager::GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace) const
{
#if WITH_EDITOR
	const FInstancedActorsInstanceHandle InstanceHandle = ActorInstanceHandleFromFSMInstanceId(InstanceId);
	if (ensure(InstanceHandle.IsValid()))
	{
		const UInstancedActorsData& InstanceData = InstanceHandle.GetInstanceActorDataChecked();
		const FTransform InstanceTransform = InstanceData.InstanceTransforms[InstanceHandle.GetIndex()];
		OutInstanceTransform = bWorldSpace ? InstanceTransform * GetActorTransform() : InstanceTransform;
		return true;
	}
#endif

	return false;
}

bool AInstancedActorsManager::SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace, bool bMarkRenderStateDirty, bool bTeleport)
{
#if WITH_EDITOR
#if WITH_EDITORONLY_DATA
	if (bLockInstanceLocation)
	{
		return false;
	}
#endif

	// Don't allow people to move an IA outside of this IAMs streaming bounds.
	FBox RuntimeBounds;
	FBox EditorBounds;
	GetStreamingBounds(RuntimeBounds, EditorBounds);
	FTransform WorldTransform = bWorldSpace ? InstanceTransform : (InstanceTransform * GetActorTransform());
	if (!RuntimeBounds.IsInside(WorldTransform.GetLocation()))
	{
#if ENABLE_DRAW_DEBUG
		FColor BoundsColor(255, 20, 20, 125);
		DrawDebugSolidBox(GetWorld(), RuntimeBounds, BoundsColor, FTransform::Identity);
		DrawDebugBox(GetWorld(), RuntimeBounds.GetCenter(), RuntimeBounds.GetExtent(), FColor::Red);
#endif
		return false;
	}

	const FInstancedActorsInstanceHandle InstanceHandle = ActorInstanceHandleFromFSMInstanceId(InstanceId);
	if (ensure(InstanceHandle.IsValid()))
	{
		InstanceHandle.GetInstanceActorDataChecked().SetInstanceTransform(InstanceHandle, InstanceTransform, bWorldSpace);
		return true;
	}
#endif

	return false;
}

void AInstancedActorsManager::NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	if (ensure(IsValid(InstanceId.ISMComponent)))
	{
		ISMInstanceManager* InstanceManager = InstanceId.ISMComponent;
		InstanceManager->NotifySMInstanceMovementStarted(InstanceId);
	}
#endif
}

void AInstancedActorsManager::NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	if (ensure(IsValid(InstanceId.ISMComponent)))
	{
		ISMInstanceManager* InstanceManager = InstanceId.ISMComponent;
		InstanceManager->NotifySMInstanceMovementOngoing(InstanceId);
	}
#endif
}

void AInstancedActorsManager::NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId)
{
#if WITH_EDITOR
	if (ensure(IsValid(InstanceId.ISMComponent)))
	{
		ISMInstanceManager* InstanceManager = InstanceId.ISMComponent;
		InstanceManager->NotifySMInstanceMovementEnded(InstanceId);
	}
#endif
}

void AInstancedActorsManager::NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected)
{
#if WITH_EDITOR
	if (ensure(IsValid(InstanceId.ISMComponent)))
	{
		ISMInstanceManager* InstanceManager = InstanceId.ISMComponent;
		InstanceManager->NotifySMInstanceSelectionChanged(InstanceId, bIsSelected);
	}
#endif
}

bool AInstancedActorsManager::DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds)
{
#if WITH_EDITOR
	Modify();
	for (const FSMInstanceId& InstanceId : InstanceIds)
	{
		if (ensure(IsValid(InstanceId.ISMComponent)))
		{
			InstanceId.ISMComponent->Modify();
		}

		const FInstancedActorsInstanceHandle InstanceHandle = ActorInstanceHandleFromFSMInstanceId(InstanceId);
		if (ensure(InstanceHandle.IsValid()))
		{
			RemoveActorInstance(InstanceHandle);
		}
	}
#endif
	return false;
}

bool AInstancedActorsManager::DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds)
{
	bool bDidDuplicate = false;
#if WITH_EDITOR
	Modify();
	for (const FSMInstanceId& InstanceId : InstanceIds)
	{
		const FInstancedActorsInstanceHandle InstanceHandle = ActorInstanceHandleFromFSMInstanceId(InstanceId);
		if (ensure(InstanceHandle.IsValid()))
		{
			UInstancedActorsData& InstanceData = InstanceHandle.GetInstanceActorDataChecked();
			const FTransform InstanceTransform = InstanceData.InstanceTransforms[InstanceHandle.GetIndex()];
			FInstancedActorsInstanceHandle IAMHandle = InstanceData.AddInstance(InstanceTransform, false);
			OutNewInstanceIds.Add(FSMInstanceId{ InstanceId.ISMComponent, IAMHandle.GetIndex() });
			bDidDuplicate = true;
		}
	}
#endif
	return bDidDuplicate;
}

FInstancedActorsInstanceHandle AInstancedActorsManager::ActorInstanceHandleFromFSMInstanceId(const FSMInstanceId& InstanceId) const
{
	if (const int32* InstanceDataID = ISMComponentToInstanceDataMap.Find(InstanceId.ISMComponent))
	{
		UInstancedActorsData* const InstanceData = FindInstanceDataByID(*InstanceDataID);
		if (ensure(InstanceData != nullptr) && ensure(InstanceId.InstanceIndex != INDEX_NONE))
		{
			return FInstancedActorsInstanceHandle(*InstanceData, FInstancedActorsInstanceIndex(InstanceId.InstanceIndex));
		}
	}

	UE_LOGF(LogInstancedActors, Error, "%ls - no Instanced Actor found for %ls index %d.", *GetPathName(), *InstanceId.ISMComponent->GetName(), InstanceId.InstanceIndex);
	return FInstancedActorsInstanceHandle();
}
//~ End ISMInstanceManager Overrides
