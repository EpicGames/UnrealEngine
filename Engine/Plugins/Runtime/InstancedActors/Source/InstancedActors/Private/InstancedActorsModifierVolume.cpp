// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsModifierVolume.h"
#include "InstancedActorsModifierVolumeComponent.h"


//-----------------------------------------------------------------------------
// AInstancedActorsModifierVolume
//-----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsModifierVolume)
AInstancedActorsModifierVolume::AInstancedActorsModifierVolume(const FObjectInitializer& ObjectInitializer)
{
	ModifierVolumeComponent = CreateDefaultSubobject<UInstancedActorsModifierVolumeComponent>(TEXT("ModifierVolume"));
	ModifierVolumeComponent->Extent = FVector(50.0f);
	ModifierVolumeComponent->Radius = 50.0f;
	RootComponent = ModifierVolumeComponent;
}

//-----------------------------------------------------------------------------
// AInstancedActorsRemovalModifierVolume
//-----------------------------------------------------------------------------
AInstancedActorsRemovalModifierVolume::AInstancedActorsRemovalModifierVolume(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer.SetDefaultSubobjectClass<URemoveInstancesModifierVolumeComponent>(TEXT("ModifierVolume")))
{
	ModifierVolumeComponent->Color = FColor::Red;
}
