// Copyright Epic Games, Inc. All Rights Reserved.


#include "InstancedActorsModifierVolumeComponent.h"
#include "InstancedActorsDebug.h"
#include "InstancedActorsIteration.h"
#include "InstancedActorsModifiers.h"
#include "InstancedActorsSubsystem.h"
#include "InstancedActorsTypes.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "Algo/NoneOf.h"
#include "Net/UnrealNetwork.h"
#include "PrimitiveSceneProxy.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "ShowFlags.h"
#include "UObject/Package.h"
#include "WorldPartition/WorldPartitionRuntimeLevelStreamingCell.h"
#include "WorldPartition/WorldPartitionLevelStreamingDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsModifierVolumeComponent)

namespace UE::InstancedActors
{
	namespace Helpers
	{
		FTopLevelAssetPath FindLevelWorldPartitionAssetPath(const ULevel* Level)
		{
			FTopLevelAssetPath AssetPath;
			if (const UWorldPartitionRuntimeLevelStreamingCell* WorldPartitionCell = Cast<UWorldPartitionRuntimeLevelStreamingCell>(Level->GetWorldPartitionRuntimeCell()))
			{
				if (const UWorldPartitionLevelStreamingDynamic* WorldPartitionLevelStreaming = Cast<UWorldPartitionLevelStreamingDynamic>(WorldPartitionCell->GetLevelStreaming()))
				{
					AssetPath = WorldPartitionLevelStreaming->GetOuterWorldPartition().GetAssetPath();
				}
			}

			return AssetPath;
		}
	} // Helpers

} // namespace InstancedActors

//-----------------------------------------------------------------------------
// UInstancedActorsModifierVolumeComponent
//-----------------------------------------------------------------------------
UInstancedActorsModifierVolumeComponent::UInstancedActorsModifierVolumeComponent(const FObjectInitializer& ObjectInitializer)
: Super(ObjectInitializer)
{
	SetCanEverAffectNavigation(false);
	SetCollisionEnabled(ECollisionEnabled::NoCollision);
}

void UInstancedActorsModifierVolumeComponent::BeginPlay()
{
	Super::BeginPlay();

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UInstancedActorsModifierVolumeComponent BeginPlay");

	// Make sure bounds are up to date e.g: on clients we will have only just received Extent replication
	// but not updated bounds via any of the normal component init mechanisms.
	UpdateBounds();

	// Register with IA subsystem if it's available already, otherwise the subsystem will collect this 
	// volume when it initializes later.
	UInstancedActorsSubsystem* PreinitializedInstancedActorSubsystem = UInstancedActorsSubsystem::Get(this);
	if (PreinitializedInstancedActorSubsystem)
	{
		// Register modifier volume with subsystem now, which will immediately call OnAddedToSubsystem
		PreinitializedInstancedActorSubsystem->AddModifierVolume(*this);	
	}
}

void UInstancedActorsModifierVolumeComponent::OnAddedToSubsystem(UInstancedActorsSubsystem& InstancedActorSubsystem, FInstancedActorsModifierVolumeHandle InModifierVolumeHandle)
{
	ModifierVolumeHandle = InModifierVolumeHandle;

	// Register modifiers with overlapping managers
	FBox BoundingBox = Bounds.GetBox();
	// Skip registration of 0 sized volumes
	if (BoundingBox.IsValid && !BoundingBox.GetSize().IsNearlyZero()) 
	{
#if WITH_INSTANCEDACTORS_DEBUG
		UE::InstancedActors::Debug::DebugDrawModifierVolumeBounds(UE::InstancedActors::Debug::CVars::DebugModifiers, *this, FColorList::LightBlue.WithAlpha(30));
#endif // WITH_INSTANCEDACTORS_DEBUG

		InstancedActorSubsystem.ForEachManager(BoundingBox, [this](AInstancedActorsManager& Manager)
		{
			// Call AddModifierVolume which calls UInstancedActorsModifierVolumeComponent::OnAddedToManager
			// back on us to add Manager to ModifiedManagers
			Manager.AddModifierVolume(*this);
			return true;
		});
	}
}

void UInstancedActorsModifierVolumeComponent::EndPlay(EEndPlayReason::Type Reason)
{
	Super::EndPlay(Reason);

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UInstancedActorsModifierVolumeComponent EndPlay");

	UInstancedActorsSubsystem* InstancedActorSubsystem = UInstancedActorsSubsystem::Get(this);
	if (ensure(InstancedActorSubsystem))
	{
		InstancedActorSubsystem->RemoveModifierVolume(ModifierVolumeHandle);

		// Remove modifiers from previously overlapped managers
		// Note: We need to iterate a copy of ModifiedManagers as RemoveModifierVolume
		// 		 will result in OnRemovedFromManager modifying ModifiedManagers
		TArray<TWeakObjectPtr<AInstancedActorsManager>> ManagersToDeregisterFrom = ModifiedManagers;
		for (TWeakObjectPtr<AInstancedActorsManager>& ModifiedManager : ManagersToDeregisterFrom)
		{
			if (ModifiedManager.IsValid())
			{
				ModifiedManager->RemoveModifierVolume(*this);
			}
		}
		check(Algo::NoneOf(ModifiedManagers, [](auto& ModifiedManager) { return ModifiedManager.Get(); })); // We shouldn't have any valid manager ptr's left as RemoveModifierVolume above should have called OnRemovedFromManager
		ModifiedManagers.Reset();
	}
}

void UInstancedActorsModifierVolumeComponent::GetLifetimeReplicatedProps(TArray< FLifetimeProperty > & OutLifetimeProps) const
{
	Super::GetLifetimeReplicatedProps(OutLifetimeProps);
	
	DOREPLIFETIME_CONDITION(UInstancedActorsModifierVolumeComponent, Radius, COND_InitialOnly);
	DOREPLIFETIME_CONDITION(UInstancedActorsModifierVolumeComponent, Extent, COND_InitialOnly);
	DOREPLIFETIME_CONDITION(UInstancedActorsModifierVolumeComponent, Shape, COND_InitialOnly);
}

void UInstancedActorsModifierVolumeComponent::OnAddedToManager(AInstancedActorsManager& Manager)
{
	check(!ModifiedManagers.Contains(&Manager));

	ModifiedManagers.Add(&Manager);

#if WITH_INSTANCEDACTORS_DEBUG
	UE::InstancedActors::Debug::DebugDrawModifierVolumeAddedToManager(UE::InstancedActors::Debug::CVars::DebugModifiers, Manager, *this);
#endif // WITH_INSTANCEDACTORS_DEBUG
}

void UInstancedActorsModifierVolumeComponent::OnRemovedFromManager(AInstancedActorsManager& Manager)
{
	check(ModifiedManagers.Contains(&Manager));

	ModifiedManagers.Remove(&Manager);
}

bool UInstancedActorsModifierVolumeComponent::TryRunPendingModifiers(AInstancedActorsManager& Manager, TBitArray<>& InOutPendingModifiers)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("UInstancedActorsModifierVolumeComponent ModifyInstances");

#if WITH_EDITOR
	// Need to do this or the PIE experience is not a valid representation of the cooked game.
	if (IsEditorOnlyObject(this, true))
	{
		return true;
	}
#endif

	if (!ensure(InOutPendingModifiers.Num() == Modifiers.Num()))
	{
		return false;
	}

	check(GetOwner());
	if (bIgnoreOwnLevelsInstances)
	{
		// If logic matches, return true as if we have successfully run all modifiers, so they don't attempt to run again.
		ULevel* ManagerLevel = Manager.GetLevel();
		ULevel* OwnerLevel = GetOwner()->GetLevel();

		// If Actors are world partitioned, the ULevel pointers will not match even if they originally came from the same level.
		// Instead, compare the FTopLevelAssetPath from their World Partition streaming cells if applicable.
		// For non-world partitioned Actors, the logic will fall to the next block to do the original ULevel pointer comparison,
		// since either ManagerLoadedPath or OwnerLoadedPath will remain !IsValid()
		if (ManagerLevel != nullptr && OwnerLevel != nullptr)
		{
			FTopLevelAssetPath ManagerLoadedPath = UE::InstancedActors::Helpers::FindLevelWorldPartitionAssetPath(ManagerLevel);
			FTopLevelAssetPath OwnerLoadedPath = UE::InstancedActors::Helpers::FindLevelWorldPartitionAssetPath(OwnerLevel);

			if (ManagerLoadedPath.IsValid() && OwnerLoadedPath.IsValid() && ManagerLoadedPath == OwnerLoadedPath)
			{
				return true;
			}
		}

		if (ManagerLevel == OwnerLevel)
		{
			return true;
		}
	}

	if (!LevelsToIgnore.IsEmpty())
	{
		UWorld* ManagerWorld = Manager.GetTypedOuter<UWorld>();
		if (ManagerWorld)
		{
			FString ManagerMapPath;
			if (const UPackage* WorldPackage = ManagerWorld->GetPackage())
			{
				ManagerMapPath = ManagerWorld->IsPlayInEditor() ? UWorld::RemovePIEPrefix(WorldPackage->GetName()) : WorldPackage->GetName();
			}

			// This a butt ugly. I can't find a better way though since our levels are renamed with a suffix for instancing reasons and there doesn't seem to be
			// a valid lower level (Asset ID, GUID, etc) check.
			if (LevelsToIgnore.ContainsByPredicate([&](const TSoftObjectPtr<UWorld>& LevelToCheck) { return ManagerMapPath.StartsWith(LevelToCheck.GetLongPackageName()); }))
			{
				// Return true to clear dirty state for Manager's modifiers
				// so we don't keep attempting to execute them.
				return true;
			}
		}
	}

	FInstancedActorsIterationContext IterationContext;
	ON_SCOPE_EXIT { IterationContext.FlushDeferredActions(); };

	// Is manager entirely inside the volume?
	bool bEnvelopesManager = false;
	switch (Shape)
	{
		// @todo Use an OrientedBox for instance inclusion testing
		case EInstancedActorsVolumeShape::Box:
		{
			const FBox QueryBounds = Bounds.GetBox();
			bEnvelopesManager = QueryBounds.IsInside(Manager.GetInstanceBounds());
			break;
		}
		case EInstancedActorsVolumeShape::Sphere:
		{
			const FSphere QueryBounds = Bounds.GetSphere();
			bEnvelopesManager = QueryBounds.IsInside(Manager.GetInstanceBounds().Min) && QueryBounds.IsInside(Manager.GetInstanceBounds().Max);

			/** 
			 @todo mikko: This is not correct. Imagine Min at 12:00, and Max at 15:00, north-east will poke out.
						Needs to check the furthest corner:

					const FVector DiffMin = Manager.GetInstanceBounds().Min - QueryBounds.Center;
					const FVector DiffMax = Manager.GetInstanceBounds().Max - QueryBounds.Center;
					const FVector FurthestCorner = QueryBounds.Center + FVector::Max(DiffMin, DiffMax);
					bEnvelopesManager = QueryBounds.IsInside(FurthestCorner);
			*/
			break;
		}
		default:
			checkNoEntry();
	}

	// Loop through pending modifiers / modifiers that haven't executed for Manager yet
	bool bRanAllPendingModifiers = true;
	for (TBitArray<>::FIterator PendingModifierIt(InOutPendingModifiers); PendingModifierIt; ++PendingModifierIt)
	{
		if (PendingModifierIt)
		{
			const int32 ModifierIndex = PendingModifierIt.GetIndex();
			if (ensure(Modifiers.IsValidIndex(ModifierIndex)))
			{
				UInstancedActorsModifierBase* Modifier = Modifiers[ModifierIndex];
				if (ensure(IsValid(Modifier)))
				{
					// Can run now?
					if (Manager.HasSpawnedEntities() || !Modifier->DoesRequireSpawnedEntities())
					{
						// Modify all instances
						if (bEnvelopesManager)
						{
							Modifier->ModifyAllInstances(Manager, IterationContext);
						}
						// Perform per-instance modification
						else
						{
							switch (Shape)
							{
								case EInstancedActorsVolumeShape::Box:
									Modifier->ModifyAllInstancesInBounds(Bounds.GetBox(), Manager, IterationContext);
									/** 
									@todo mikko: These will test based on if the bounds overlaps with an instance.
										Why are the tests different? If the modifier bounds contains the whole manager 
										(assuming manager bounds is combination of all instance bounds), then we should 
										not need the second test at all. Maybe the manager test should be also overlap, not containment?
									*/
									break;
								case EInstancedActorsVolumeShape::Sphere:
									Modifier->ModifyAllInstancesInBounds(Bounds.GetSphere(), Manager, IterationContext);
									break;
								default:
									checkNoEntry();
							}
						}

						// Clear pending / dirty state
						PendingModifierIt.GetValue() = false;
					}
					else
					{
						bRanAllPendingModifiers = false;
					}
				}
			}
		}
	}

	return bRanAllPendingModifiers;
}

FBoxSphereBounds UInstancedActorsModifierVolumeComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	FBoxSphereBounds LocalBounds; 
	switch (Shape)
	{
		case EInstancedActorsVolumeShape::Box:
			LocalBounds = FBoxSphereBounds(FBox(-Extent, Extent));
			break;
		case EInstancedActorsVolumeShape::Sphere:
			LocalBounds = FBoxSphereBounds(FSphere(FVector::ZeroVector, Radius));
			break;
		default:
			checkNoEntry();
	}
	return LocalBounds.TransformBy(LocalToWorld);
}

FPrimitiveSceneProxy* UInstancedActorsModifierVolumeComponent::CreateSceneProxy()
{
	class FInstancedActorsModifierVolumeComponent final : public FPrimitiveSceneProxy
	{
	public:
		SIZE_T GetTypeHash() const override
		{
			static size_t UniquePointer;
			return reinterpret_cast<size_t>(&UniquePointer);
		}

		FInstancedActorsModifierVolumeComponent(const UInstancedActorsModifierVolumeComponent* InComponent)
			: FPrimitiveSceneProxy(InComponent)
			, Shape( InComponent->Shape )
			, Extent( InComponent->Extent )
			, Radius( InComponent->Radius )
			, Color( InComponent->Color )
			, LineThickness( InComponent->LineThickness )
			, bDrawOnlyIfSelected( InComponent->bDrawOnlyIfSelected )
		{
			bWillEverBeLit = false;
		}

		virtual void GetDynamicMeshElements(const TArray<const FSceneView*>& Views, const FSceneViewFamily& ViewFamily, uint32 VisibilityMap, FMeshElementCollector& Collector) const override
		{
			QUICK_SCOPE_CYCLE_COUNTER( STAT_BoxSceneProxy_GetDynamicMeshElements );

			const FMatrix& LocalToWorld = GetLocalToWorld();
			
			for (int32 ViewIndex = 0; ViewIndex < Views.Num(); ViewIndex++)
			{
				if (VisibilityMap & (1 << ViewIndex))
				{
					const FSceneView* View = Views[ViewIndex];

					const FLinearColor DrawColor = GetViewSelectionColor(Color, *View, IsSelected(), IsHovered(), false, IsIndividuallySelected() );

					FPrimitiveDrawInterface* PDI = Collector.GetPDI(ViewIndex);

					switch (Shape)
					{
						case EInstancedActorsVolumeShape::Box:
						{
							FBox BoxBounds = GetBounds().GetBox();
							DrawWireBox(PDI, BoxBounds, DrawColor, SDPG_World, LineThickness);
							break;
						}
						case EInstancedActorsVolumeShape::Sphere:
						{
							FSphere SphereBounds = GetBounds().GetSphere();
							DrawWireSphereAutoSides(PDI, SphereBounds.Center, DrawColor, SphereBounds.W, SDPG_World, LineThickness);
							break;
						}
					}
				}
			}
		}

		virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override
		{
			const bool bProxyVisible = !bDrawOnlyIfSelected || IsSelected();

			const bool bShowVolumes = View->Family->EngineShowFlags.Volumes;

			FPrimitiveViewRelevance Result;
			Result.bDrawRelevance = bShowVolumes && IsShown(View) && bProxyVisible;
			Result.bDynamicRelevance = true;
			Result.bShadowRelevance = IsShadowCast(View);
			Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
			return Result;
		}
		virtual uint32 GetMemoryFootprint( void ) const override { return( sizeof( *this ) + GetAllocatedSize() ); }
		uint32 GetAllocatedSize( void ) const { return( FPrimitiveSceneProxy::GetAllocatedSize() ); }

	private:
		const EInstancedActorsVolumeShape Shape;
		const FVector Extent;
		const float Radius;

		const FColor Color;
		const float LineThickness;
		const uint32 bDrawOnlyIfSelected:1;
	};

	return new FInstancedActorsModifierVolumeComponent( this );
}

//-----------------------------------------------------------------------------
// URemoveInstancesModifierVolumeComponent
//-----------------------------------------------------------------------------
URemoveInstancesModifierVolumeComponent::URemoveInstancesModifierVolumeComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	URemoveInstancedActorsModifier* RemoveInstancedActorsModifier = CreateDefaultSubobject<URemoveInstancedActorsModifier>(TEXT("RemoveInstancedActorsModifier"));
	Modifiers.Add(RemoveInstancedActorsModifier);
}
