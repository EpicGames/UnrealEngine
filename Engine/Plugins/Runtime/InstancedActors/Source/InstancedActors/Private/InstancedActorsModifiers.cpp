// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsModifiers.h"
#include "InstancedActorsDebug.h"
#include "InstancedActorsIndex.h"
#include "InstancedActorsIteration.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"
#include "DrawDebugHelpers.h"
#include "Math/Box.h"
#include "Math/Sphere.h"


//-----------------------------------------------------------------------------
// UInstancedActorsModifierBase
//-----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsModifiers)
void UInstancedActorsModifierBase::ModifyAllInstances(AInstancedActorsManager& Manager, FInstancedActorsIterationContext& IterationContext)
{
	Manager.ForEachInstance([this](const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext)
	{
		return ModifyInstance(InstanceHandle, InstanceTransform, IterationContext);
	}, 
	IterationContext,
	/*Predicate*/InstanceTagsQuery.IsEmpty() ? TOptional<AInstancedActorsManager::FInstancedActorDataPredicateFunc>() : TOptional<AInstancedActorsManager::FInstancedActorDataPredicateFunc>([this](const UInstancedActorsData& InstancedActorData)
	{
		return InstancedActorData.GetCombinedTags().MatchesQuery(InstanceTagsQuery);
	}));
}

//-----------------------------------------------------------------------------
// URemoveInstancedActorsModifier
//-----------------------------------------------------------------------------
URemoveInstancedActorsModifier::URemoveInstancedActorsModifier()
{
	// Safe * more efficient to perform deletions before entity spawning
	bRequiresSpawnedEntities = false;
}

void URemoveInstancedActorsModifier::ModifyAllInstances(AInstancedActorsManager& Manager, FInstancedActorsIterationContext& IterationContext)
{
#if WITH_INSTANCEDACTORS_DEBUG
	// Debug draw removed instances
	UE::InstancedActors::Debug::DebugDrawAllInstanceLocations(UE::InstancedActors::Debug::CVars::DebugModifiers, ELogVerbosity::Verbose, Manager, FColor::Red, /*LogOwner*/&Manager, "LogInstancedActors");
#endif

	if (InstanceTagsQuery.IsEmpty())
	{
		IterationContext.RemoveAllInstancesDeferred(Manager);
	}
	else
	{
		for (const TObjectPtr<UInstancedActorsData>& InstanceData : Manager.GetAllInstanceData())
		{
			check(IsValid(InstanceData));

			if (InstanceData->GetCombinedTags().MatchesQuery(InstanceTagsQuery))
			{
				IterationContext.RemoveAllInstancesDeferred(*InstanceData);
			}
		}
	}
}

bool URemoveInstancedActorsModifier::ModifyInstance(const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext)
{
#if WITH_INSTANCEDACTORS_DEBUG
	// Debug draw removed instances
	UE::InstancedActors::Debug::DebugDrawLocation(UE::InstancedActors::Debug::CVars::DebugModifiers, InstanceHandle.GetManager()->GetWorld(), /*LogOwner*/InstanceHandle.GetManager(), LogInstancedActors, ELogVerbosity::Verbose, InstanceTransform.GetLocation(), /*Size*/30.0f, FColor::Red, TEXT("%d"), InstanceHandle.GetIndex());
#endif

	IterationContext.RemoveInstanceDeferred(InstanceHandle);

	return true;
}
