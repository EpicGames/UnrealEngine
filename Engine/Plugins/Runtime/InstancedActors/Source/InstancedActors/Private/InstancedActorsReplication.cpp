// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsReplication.h"
#include "InstancedActorsData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsReplication)


void FInstancedActorsDeltaList::Initialize(UInstancedActorsData& InOwnerInstancedActorData)
{
	const uintptr_t ThisStart = (uintptr_t)this;
	const uintptr_t ThisEnd = ThisStart + sizeof(FInstancedActorsDeltaList);
	const uintptr_t OwnerStart = (uintptr_t)&InOwnerInstancedActorData;
	const uintptr_t OwnerEnd = OwnerStart + InOwnerInstancedActorData.GetClass()->GetStructureSize();

	if (OwnerStart <= ThisStart && ThisEnd <= OwnerEnd)
	{
		InstancedActorData = &InOwnerInstancedActorData;
	}
	else
	{
		static constexpr TCHAR MessageFormat[] = TEXT("InOwnerInstancedActorData is required to be the instance containing `this` as a member. Ignoring.");
		checkf(false, MessageFormat);
		UE_LOG(LogInstancedActors, Error, MessageFormat);
	}
}

FInstancedActorsDelta& FInstancedActorsDeltaList::FindOrAddInstanceDelta(FInstancedActorsInstanceIndex InstanceIndex)
{
	uint16& DeltaIndex = InstanceIndexToDeltaIndex.FindOrAdd(InstanceIndex, (uint16)INDEX_NONE);
	if (DeltaIndex == (uint16)INDEX_NONE)
	{
		const int32 NewDeltaIndex = InstanceDeltas.Emplace(InstanceIndex);
		checkf(NewDeltaIndex < std::numeric_limits<uint16>::max(), TEXT("Reach limit of supported deltas"));
		DeltaIndex = NewDeltaIndex;
	}

	check(InstanceDeltas.IsValidIndex(DeltaIndex));
	return InstanceDeltas[DeltaIndex];
}

void FInstancedActorsDeltaList::RemoveInstanceDelta(uint16 DeltaIndex)
{
	check(InstanceDeltas.IsValidIndex(DeltaIndex));
	FInstancedActorsDelta& InstanceDelta = InstanceDeltas[DeltaIndex];

	uint16* DeltaIndexPtr = InstanceIndexToDeltaIndex.Find(InstanceDelta.GetInstanceIndex());
	if (ensureMsgf(DeltaIndexPtr && *DeltaIndexPtr == DeltaIndex, TEXT("Expecting the instance to exist and match the delta index")))
	{
		InstanceIndexToDeltaIndex.Remove(InstanceDelta.GetInstanceIndex());

		InstanceDeltas.RemoveAtSwap(DeltaIndex);
		MarkArrayDirty();

		// Fix up swapped-in delta's index lookup
		if (InstanceDeltas.IsValidIndex(DeltaIndex))
		{
			FInstancedActorsDelta& SwappedInInstanceDelta = InstanceDeltas[DeltaIndex];
			uint16* SwappedInInstanceDeltaIndex = InstanceIndexToDeltaIndex.Find(SwappedInInstanceDelta.GetInstanceIndex());
			if (ensureMsgf(SwappedInInstanceDeltaIndex, TEXT("InstanceIndexToDeltaIndex and InstanceDeltas have gotten out of sync! Couldn't find delta index for swapped in instance delta %s (RemoveAtSwap'd index: %u)"), *SwappedInInstanceDelta.GetInstanceIndex().GetDebugName(), DeltaIndex))
			{
				*SwappedInInstanceDeltaIndex = DeltaIndex;
			}
		}
	}
}

bool FInstancedActorsDeltaList::NetDeltaSerialize(FNetDeltaSerializeInfo& NetDeltaParams)
{
	return FFastArraySerializer::FastArrayDeltaSerialize<FInstancedActorsDelta, FInstancedActorsDeltaList>(InstanceDeltas, NetDeltaParams, *this);
}

// @todo consider merging the data in these two call backs (also PostReplicatedChange()) see CallPostReplicatedReceiveOrNot()
void FInstancedActorsDeltaList::PostReplicatedAdd(const TArrayView<int32>& AddedIndices, int32 FinalSize)
{
	if (ensure(InstancedActorData) && AddedIndices.Num() > 0)
	{
		InstancedActorData->OnRep_InstanceDeltas(TConstArrayView<int32>(AddedIndices));
	}
}

void FInstancedActorsDeltaList::PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize)
{
	if (ensure(InstancedActorData) && ChangedIndices.Num() > 0)
	{
		InstancedActorData->OnRep_InstanceDeltas(TConstArrayView<int32>(ChangedIndices));
	}
}

void FInstancedActorsDeltaList::PreReplicatedRemove(const TArrayView<int32>& RemovedIndices, int32 FinalSize)
{
	if (ensure(InstancedActorData) && RemovedIndices.Num() > 0)
	{
		InstancedActorData->OnRep_PreRemoveInstanceDeltas(TConstArrayView<int32>(RemovedIndices));
	}
}

void FInstancedActorsDeltaList::SetInstanceDestroyed(FInstancedActorsInstanceIndex InstanceIndex)
{
	FInstancedActorsDelta& InstanceDelta = FindOrAddInstanceDelta(InstanceIndex);
	if (ensure(!InstanceDelta.IsDestroyed()))
	{
		InstanceDelta.SetDestroyed(true);
		++NumDestroyedInstanceDeltas;
		MarkItemDirty(InstanceDelta);
	}
}

void FInstancedActorsDeltaList::RemoveDestroyedInstanceDelta(FInstancedActorsInstanceIndex InstanceIndex)
{
	uint16* DeltaIndexPtr = InstanceIndexToDeltaIndex.Find(InstanceIndex);
	if (DeltaIndexPtr != nullptr)
	{
		uint16 DeltaIndex = *DeltaIndexPtr;

		if (ensureMsgf(InstanceDeltas.IsValidIndex(DeltaIndex), TEXT("Expecting a valid delta index")))
		{
			FInstancedActorsDelta& InstanceDelta = InstanceDeltas[DeltaIndex];
			if (ensureMsgf(InstanceDelta.GetInstanceIndex() == InstanceIndex, TEXT("Expecting instance index to match")))
			{
				if (InstanceDelta.IsDestroyed())
				{
					InstanceDelta.SetDestroyed(false);

					if (!InstanceDelta.HasAnyDeltas())
					{
						RemoveInstanceDelta(DeltaIndex);
					}

					--NumDestroyedInstanceDeltas;
				}
			}
		}
	}
}

void FInstancedActorsDeltaList::SetCurrentLifecyclePhaseIndex(FInstancedActorsInstanceIndex InstanceIndex, uint8 InCurrentLifecyclePhaseIndex)
{
	FInstancedActorsDelta& InstanceDelta = FindOrAddInstanceDelta(InstanceIndex);
	if (InstanceDelta.GetCurrentLifecyclePhaseIndex() != InCurrentLifecyclePhaseIndex)
	{
		if (!InstanceDelta.HasCurrentLifecyclePhase())
		{
			++NumLifecyclePhaseDeltas;
		}
		InstanceDelta.SetCurrentLifecyclePhaseIndex(InCurrentLifecyclePhaseIndex);
		MarkItemDirty(InstanceDelta);
	}
}	

void FInstancedActorsDeltaList::RemoveLifecyclePhaseDelta(FInstancedActorsInstanceIndex InstanceIndex)
{
	uint16* DeltaIndexPtr = InstanceIndexToDeltaIndex.Find(InstanceIndex);
	if (DeltaIndexPtr != nullptr)
	{
		uint16 DeltaIndex = *DeltaIndexPtr;

		if (ensureMsgf(InstanceDeltas.IsValidIndex(DeltaIndex), TEXT("Expecting a valid delta index")))
		{
			FInstancedActorsDelta& InstanceDelta = InstanceDeltas[DeltaIndex];
			if (ensureMsgf(InstanceDelta.GetInstanceIndex() == InstanceIndex, TEXT("Expecting instance index to match")))
			{
				if (InstanceDelta.HasCurrentLifecyclePhase())
				{
					InstanceDelta.ResetLifecyclePhaseIndex();

					if (!InstanceDelta.HasAnyDeltas())
					{
						RemoveInstanceDelta(DeltaIndex);
					}
					else
					{
						MarkItemDirty(InstanceDelta);
					}

					--NumLifecyclePhaseDeltas;
				}
			}
		}
	}
}

#if WITH_SERVER_CODE
void FInstancedActorsDeltaList::SetCurrentLifecyclePhaseTimeElapsed(FInstancedActorsInstanceIndex InstanceIndex, FFloat16 InCurrentLifecyclePhaseTimeElapsed)
{
	FInstancedActorsDelta& InstanceDelta = FindOrAddInstanceDelta(InstanceIndex);
	if (InstanceDelta.GetCurrentLifecyclePhaseTimeElapsed() != InCurrentLifecyclePhaseTimeElapsed)
	{
		if (!InstanceDelta.HasCurrentLifecyclePhaseTimeElapsed())
		{
			++NumLifecyclePhaseTimeElapsedDeltas;
		}
		InstanceDelta.SetCurrentLifecyclePhaseTimeElapsed(InCurrentLifecyclePhaseTimeElapsed);
		// No need to MarkItemDirty(InstanceDelta) as this is not a replicated field
	}
}

void FInstancedActorsDeltaList::RemoveLifecyclePhaseTimeElapsedDelta(FInstancedActorsInstanceIndex InstanceIndex)
{
	uint16* DeltaIndexPtr = InstanceIndexToDeltaIndex.Find(InstanceIndex);
	if (DeltaIndexPtr != nullptr)
	{
		uint16 DeltaIndex = *DeltaIndexPtr;

		if (ensureMsgf(InstanceDeltas.IsValidIndex(DeltaIndex), TEXT("Expecting a valid delta index")))
		{
			FInstancedActorsDelta& InstanceDelta = InstanceDeltas[DeltaIndex];
			if (ensureMsgf(InstanceDelta.GetInstanceIndex() == InstanceIndex, TEXT("Expecting instance index to match")))
			{
				if (InstanceDelta.HasCurrentLifecyclePhaseTimeElapsed())
				{
					InstanceDelta.ResetLifecyclePhaseTimeElapsed();

					if (!InstanceDelta.HasAnyDeltas())
					{
						RemoveInstanceDelta(DeltaIndex);
					}

					--NumLifecyclePhaseTimeElapsedDeltas;
				}
			}
		}
	}
}
#endif // WITH_SERVER_CODE

void FInstancedActorsDeltaList::Reset(bool bMarkDirty)
{
	InstanceDeltas.Reset();
	InstanceIndexToDeltaIndex.Reset();

	NumDestroyedInstanceDeltas = 0;
	NumLifecyclePhaseDeltas = 0;
	NumLifecyclePhaseTimeElapsedDeltas = 0;

	if (bMarkDirty)
	{
		MarkArrayDirty();
	}
}
