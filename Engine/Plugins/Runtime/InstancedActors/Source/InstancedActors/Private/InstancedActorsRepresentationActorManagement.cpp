// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsRepresentationActorManagement.h"
#include "InstancedActorsData.h"
#include "InstancedActorsSubsystem.h"
#include "InstancedActorsSettingsTypes.h"
#include "InstancedActorsVisualizationProcessor.h"
#include "Delegates/Delegate.h"
#include "MassEntitySubsystem.h"
#include "MassActorSpawnerSubsystem.h"
#include "MassCommandBuffer.h"
#include "MassCommands.h"
#include "MassEntityView.h"
#include "MassRepresentationTypes.h"
#include "MassCommonFragments.h"
#include "MassRepresentationSubsystem.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationProcessor.h"
#include "MassStationaryISMSwitcherProcessor.h"
#include "MassDebugger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsRepresentationActorManagement)


EMassActorSpawnRequestAction UInstancedActorsRepresentationActorManagement::OnPostActorSpawn(const FMassActorSpawnRequestHandle& SpawnRequestHandle
	, FConstStructView SpawnRequest, TSharedRef<FMassEntityManager> EntityManager) const
{
	const EMassActorSpawnRequestAction ResultAction = Super::OnPostActorSpawn(SpawnRequestHandle, SpawnRequest, EntityManager);

	if (ResultAction == EMassActorSpawnRequestAction::Keep)
	{
		const FMassActorSpawnRequest& MassActorSpawnRequest = SpawnRequest.Get<const FMassActorSpawnRequest>();
		if (AActor* Actor = Cast<AActor>(MassActorSpawnRequest.SpawnedActor))
		{
			if (USceneComponent* RootComponent = Actor->GetRootComponent())
			{
				RootComponent->TransformUpdated.AddUObject(this, &UInstancedActorsRepresentationActorManagement::OnSpawnedActorMoved, MassActorSpawnRequest.MassAgent);
			}
		}

		// Allow settings to turn off damage for this actor.
		UInstancedActorsSubsystem& InstancedActorSubsystem = UInstancedActorsSubsystem::GetChecked(MassActorSpawnRequest.SpawnedActor);
		FSharedStruct SharedSettings = InstancedActorSubsystem.GetOrCompileSettingsForActorClass(MassActorSpawnRequest.SpawnedActor->GetClass());
		const FInstancedActorsSettings* Settings = SharedSettings.GetPtr<FInstancedActorsSettings>();
		if (Settings && Settings->bOverride_bCanBeDamaged)
		{
			MassActorSpawnRequest.SpawnedActor->SetCanBeDamaged(Settings->bCanBeDamaged);
		}
	}
		
	return ResultAction;
}

void UInstancedActorsRepresentationActorManagement::SetActorEnabled(const EMassActorEnabledType EnabledType, AActor& Actor, const int32 EntityIdx, FMassCommandBuffer& CommandBuffer) const
{
	// On clients, we don't want to mess with Actors at all, deferring to server replication.
	// @todo Ultimately this actually shouldn't be called at all, as clients shouldn't even be trying to switch from Actor to ISMC LOD.
	//       Rather, we should be forcing Actor LOD whilst we have an actor present on the client and only switch to ISMC without one.
	if (!Actor.HasAuthority())
	{
		return;
	}

	if (EnabledType != EMassActorEnabledType::Disabled)
	{
		if (Actor.GetActorEnableCollision())
		{
			// Deferring this as there is a callback internally that could end up doing things outside of the game thread and will fire checks(Chaos mostly)
			CommandBuffer.PushCommand<FMassDeferredSetCommand>([WeakActor = TWeakObjectPtr<AActor>(&Actor)](FMassEntityManager&)
				{
					if (AActor* ActorPtr = WeakActor.Get())
					{
						ActorPtr->SetActorEnableCollision(true);
					}
				});
		}

		// mz@todo IA: reconsider for generic use, should probably be configurable.
		// Don't call Super::SetActorEnabled when enabling, to skip default implementation of enabling tick which we don't want to do
	}
	else
	{
		Super::SetActorEnabled(EnabledType, Actor, EntityIdx, CommandBuffer);
	}
}

void UInstancedActorsRepresentationActorManagement::TeleportActor(const FTransform& Transform, AActor& Actor, FMassCommandBuffer& CommandBuffer) const
{
	// On clients, the replicated actors transform will be slightly different to the pure Mass instance transform, due to net quantization.
	// Importantly, this is enough to trigger a transform update in Super::TeleportActor when it's called from UMassRepresentationProcessor::UpdateRepresentation
	// which can liven physics and such, so we simply skip this here, assuming the authoritative transform is correct enough.
	if (!Actor.HasAuthority())
	{
#if DO_CHECK
		ensureMsgf(LIKELY(Actor.GetTransform().Equals(Transform, 5.0f)), TEXT("Replicated instanced actor (%s) transform has an unexpectedly large difference to it's cooked instance transform"), *Actor.GetPathName());
#endif
		return;
	}

	Super::TeleportActor(Transform, Actor, CommandBuffer);
}

void UInstancedActorsRepresentationActorManagement::OnActorUnlinked(AActor& Actor)
{
	// Disconnect from delegates we subscribed to in OnPostActorSpawn
	if (USceneComponent* RootComponent = Actor.GetRootComponent())
	{
		RootComponent->TransformUpdated.RemoveAll(this);
	}
}

void UInstancedActorsRepresentationActorManagement::OnSpawnedActorDestroyed(AActor& DestroyedActor, FMassEntityHandle EntityHandle) const
{
	UWorld* World = DestroyedActor.GetWorld();
	if (ensure(World))
	{
		UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>();
		if (ensure(EntitySubsystem))
		{
			FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();
			if (ensureMsgf(EntityManager.IsEntityActive(EntityHandle), TEXT("Received player-instigated destruction event for an invalid entity")))
			{
				UInstancedActorsData* InstanceData = UInstancedActorsData::GetInstanceDataForEntity(EntityManager, EntityHandle);
				if (ensure(InstanceData))
				{
					InstanceData->OnInstancedActorDestroyed(DestroyedActor, EntityHandle);
				}
			}
		}
	}
}

void UInstancedActorsRepresentationActorManagement::OnSpawnedActorMoved(USceneComponent* MovedActorRootComponent, EUpdateTransformFlags TransformUpdateFlags, ETeleportType TeleportType, FMassEntityHandle EntityHandle) const
{
	check(IsValid(MovedActorRootComponent));

	AActor* MovedActor = MovedActorRootComponent->GetOwner();
	UWorld* World = MovedActor->GetWorld();
	if (ensure(World))
	{
		UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>();
		if (ensure(EntitySubsystem))
		{
			FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();
			if (ensureMsgf(EntityManager.IsEntityActive(EntityHandle), TEXT("Received actor movement event for an invalid entity")))
			{
				UInstancedActorsData* InstanceData = UInstancedActorsData::GetInstanceDataForEntity(EntityManager, EntityHandle);
				if (ensure(InstanceData))
				{
					InstanceData->OnInstancedActorMoved(*MovedActor, EntityHandle);
				}
			}
		}
	}
}

AActor* UInstancedActorsRepresentationActorManagement::FindOrInstantlySpawnActor(UMassRepresentationSubsystem& RepresentationSubsystem, FMassEntityManager& EntityManager, FMassEntityView& EntityView)
{
	const FTransformFragment& TransformFragment = EntityView.GetFragmentData<FTransformFragment>();
	FMassRepresentationFragment& Representation = EntityView.GetFragmentData<FMassRepresentationFragment>();
	
	AActor* SpawnedActor = GetOrSpawnActor(RepresentationSubsystem, EntityManager, EntityView.GetEntity()
		, TransformFragment.GetTransform(), Representation.HighResTemplateActorIndex, Representation.ActorSpawnRequestHandle
		, /*Priority=*/0);

	if (!SpawnedActor)
	{
		// force spawn
		UMassActorSpawnerSubsystem* ActorSpawnerSubsystem = RepresentationSubsystem.GetActorSpawnerSubsystem();
		check(ActorSpawnerSubsystem);
		const ESpawnRequestStatus RequestResult = ActorSpawnerSubsystem->ProcessSpawnRequest(Representation.ActorSpawnRequestHandle);
		
		if (RequestResult == ESpawnRequestStatus::Failed || RequestResult == ESpawnRequestStatus::Succeeded)
		{
			if (RequestResult == ESpawnRequestStatus::Succeeded)
			{
				FMassActorSpawnRequest& SpawnRequest = ActorSpawnerSubsystem->GetMutableSpawnRequest<FMassActorSpawnRequest>(Representation.ActorSpawnRequestHandle);
				SpawnedActor = SpawnRequest.SpawnedActor;
				ensureMsgf(ActorSpawnerSubsystem->RemoveActorSpawnRequest(Representation.ActorSpawnRequestHandle), TEXT("Unable to remove a valid spawn request"));
			}
			
			Representation.ActorSpawnRequestHandle.Invalidate();
		}
	}

	check(EntityManager.IsProcessing() == false);

	if (SpawnedActor)
	{
		FMassRepresentationLODFragment& RepresentationLOD = EntityView.GetFragmentData<FMassRepresentationLODFragment>();
		RepresentationLOD.LOD = EMassLOD::High;

		// ALSO: need to hide the ISM instance
		// @todo Duplicated Code could be solved with a "trait" pattern - collect all relevant classes in one declaration per mesh-type and use that. 
		if (Representation.CurrentRepresentation == EMassRepresentationType::StaticMeshInstance)
		{
			FMassInstancedStaticMeshInfoArrayView ISMInfosView = RepresentationSubsystem.GetMutableInstancedStaticMeshInfos();

			const int32 ISMInfoIndex = Representation.StaticMeshDescHandle.ToIndex();
			if (ensureMsgf(ISMInfosView.IsValidIndex(ISMInfoIndex), TEXT("Invalid handle index %u for ISMInfosView"), ISMInfoIndex))
			{
				FMassInstancedStaticMeshInfo& ISMInfo = ISMInfosView[ISMInfoIndex];

				ISMInfo.RemoveInstance(EntityView.GetEntity(), Representation.PrevLODSignificance);
				ISMInfo.RemoveInstance(EntityView.GetEntity(), RepresentationLOD.LODSignificance);
			}
		}
		else if (Representation.CurrentRepresentation == EMassRepresentationType::SkinnedMeshInstance)
		{
			FMassInstancedSkinnedMeshInfoArrayView SkinnedMeshInfosView = RepresentationSubsystem.GetMutableInstancedSkinnedMeshInfos();

			const int32 SkinnedMeshInfoIndex = Representation.SkinnedMeshDescHandle.ToIndex();
			if (ensureMsgf(SkinnedMeshInfosView.IsValidIndex(SkinnedMeshInfoIndex), TEXT("Invalid handle index %u for SkinnedMeshInfosView"), SkinnedMeshInfoIndex))
			{
				FMassInstancedSkinnedMeshInfo& SkinnedMeshInfo = SkinnedMeshInfosView[SkinnedMeshInfoIndex];

				SkinnedMeshInfo.RemoveInstance(EntityView.GetEntity(), Representation.PrevLODSignificance);
				SkinnedMeshInfo.RemoveInstance(EntityView.GetEntity(), RepresentationLOD.LODSignificance);
			}
		}

		Representation.PrevRepresentation = Representation.CurrentRepresentation;
		Representation.CurrentRepresentation = EMassRepresentationType::HighResSpawnedActor;

		// note that we do need to make the change synchronously since due to EntityManager.IsProcessing() == false
		// any command we issue here might get called after LOD and Visualization processing, that could override the
		// values we've just set
		FMassTagBitSet TagsToRemove = UE::Mass::Utils::ConstructTagBitSet<EMassCommandCheckTime::CompileTimeCheck, FMassStationaryISMSwitcherProcessorTag, FInstancedActorsVisualizationProcessorTag>();
		EntityManager.RemoveCompositionFromEntity(EntityView.GetEntity(), FMassArchetypeCompositionDescriptor(MoveTemp(TagsToRemove)));
	}

	return SpawnedActor;
}
