// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsRepresentationSubsystem.h"
#include "InstancedActorsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsRepresentationSubsystem)


void UInstancedActorsRepresentationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	TSubclassOf<UMassActorSpawnerSubsystem> SpawnerSystemSubclass = UE::InstancedActors::Utils::DetermineActorSpawnerSubsystemClass(GetWorldRef());
	if (SpawnerSystemSubclass)
	{
		ActorSpawnerSubsystem = Cast<UMassActorSpawnerSubsystem>(Collection.InitializeDependency(SpawnerSystemSubclass));

		ensureMsgf(ActorSpawnerSubsystem, TEXT("Trying to initialize dependency on class %s failed. Verify InstanedActors settings.")
			, *SpawnerSystemSubclass->GetName());
	}

	OnSettingsChangedHandle = GET_INSTANCEDACTORS_CONFIG_VALUE(GetOnSettingsUpdated()).AddUObject(this, &UInstancedActorsRepresentationSubsystem::OnSettingsChanged);
}

void UInstancedActorsRepresentationSubsystem::Deinitialize()
{
	GET_INSTANCEDACTORS_CONFIG_VALUE(GetOnSettingsUpdated()).Remove(OnSettingsChangedHandle);
	ActorSpawnerSubsystem = nullptr;

	Super::Deinitialize();
}

void UInstancedActorsRepresentationSubsystem::OnSettingsChanged()
{
	if (UWorld* World = GetWorld())
	{
		ActorSpawnerSubsystem = UE::InstancedActors::Utils::GetActorSpawnerSubsystem(*World);
		UE_CLOGF(ActorSpawnerSubsystem == nullptr, LogInstancedActors, Warning
			, "%ls %s failed to fetch ActorSpawnerSubsystem instance, class %ls."
			, *GetName(), __FUNCTION__, *GetNameSafe(UE::InstancedActors::Utils::DetermineActorSpawnerSubsystemClass(*World)));
	}
}
