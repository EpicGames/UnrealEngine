// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsRuntimeRemovalProcessor.h"
#include "InstancedActorsData.h"
#include "InstancedActorsStationaryLODBatchProcessor.h"
#include "InstancedActorsSubsystem.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "MassRepresentationFragments.h"
#include "MassStationaryISMSwitcherProcessor.h"
#include "InstancedActorsSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsRuntimeRemovalProcessor)

UInstancedActorsRuntimeRemovalProcessor::UInstancedActorsRuntimeRemovalProcessor()
: EntityQuery(*this)
{
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Representation;
	bAutoRegisterWithProcessingPhases = true;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllNetModes);

	ExecutionOrder.ExecuteBefore.Add(UInstancedActorsStationaryLODBatchProcessor::StaticClass()->GetFName());
	ExecutionOrder.ExecuteBefore.Add(UMassStationaryISMSwitcherProcessor::StaticClass()->GetFName());
}

void UInstancedActorsRuntimeRemovalProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	// Not directly used, but kept for the dependency solver
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);

	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		if (TSubclassOf<UInstancedActorsSubsystem> InstancedActorsSubsystemClass = GET_INSTANCEDACTORS_CONFIG_VALUE(GetInstancedActorsSubsystemClass()))
		{
			ProcessorRequirements.AddSubsystemRequirement(InstancedActorsSubsystemClass, EMassFragmentAccess::ReadWrite, EntityManager);
		}
		else
		{
			UE_LOGF(LogInstancedActors, Error, "Misconfigured UInstancedActorsSubsystem subclass");
		}
	}
}

void UInstancedActorsRuntimeRemovalProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	TSubclassOf<UInstancedActorsSubsystem> InstancedActorsSubsystemClass = GET_INSTANCEDACTORS_CONFIG_VALUE(GetInstancedActorsSubsystemClass());
	if (!ensureMsgf(InstancedActorsSubsystemClass, TEXT("Misconfigured UInstancedActorsSubsystem subclass")))
	{
		return;
	}
	//@todo: revisit this temporary fix. This was done because UGameFeatureAction_ConfigureInstancedActors::OnGameFeatureActivating() may happen after UInstancedActorsRuntimeRemovalProcessor::ConfigureQueries()
	//UInstancedActorsSubsystem* InstancedActorSubsystem = Context.GetMutableSubsystem<UInstancedActorsSubsystem>(InstancedActorsSubsystemClass);
	UInstancedActorsSubsystem* InstancedActorsSubsystem = UE::InstancedActors::Utils::GetInstancedActorsSubsystem(*Context.GetWorld());
	if (!ensureMsgf(InstancedActorsSubsystem, TEXT("UInstancedActorsSubsystem is missing, this is unexpected")))
	{
		return;
	}

	if (!InstancedActorsSubsystem->ContainsDirtyIADWithEntitiesPendingDestruction())
	{
		return;
	}

	TArray<TWeakObjectPtr<UInstancedActorsData>> DirtyIADs;
	TArray<FMassEntityHandle> EntitiesPendingDestruction;
	InstancedActorsSubsystem->PopAllDirtyIADWithEntitiesPendingDestruction(DirtyIADs);

	for (const TWeakObjectPtr<UInstancedActorsData>& DirtyIAD : DirtyIADs)
	{
		if (UInstancedActorsData* InstanceData = DirtyIAD.Get())
		{
			InstanceData->PopAllEntitiesPendingDestruction(EntitiesPendingDestruction);
			if (EntitiesPendingDestruction.Num() > 0)
			{
				for (const FMassEntityHandle& EntityToDestroy : EntitiesPendingDestruction)
				{
					if (EntityManager.IsEntityValid(EntityToDestroy))
					{
						if (FMassRepresentationFragment* Representation = EntityManager.GetFragmentDataPtr<FMassRepresentationFragment>(EntityToDestroy))
						{
							Representation->bIsPendingDestruction = true;
						}
					}
				}

				Context.Defer().DestroyEntities(MoveTemp(EntitiesPendingDestruction));
			}
		}
	}
}
