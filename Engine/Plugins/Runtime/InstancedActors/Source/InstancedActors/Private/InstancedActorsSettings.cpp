// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsSettings.h"
#include "ClientInstancedActorsSpawnerSubsystem.h"
#include "ServerInstancedActorsSpawnerSubsystem.h"
#include "InstancedActorsSubsystem.h"
#include "InstancedActorsVisualizationTrait.h"


//-----------------------------------------------------------------------------
// UInstancedActorsProjectSettings
//-----------------------------------------------------------------------------

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsSettings)
UInstancedActorsProjectSettings::UInstancedActorsProjectSettings()
{
	DefaultConfig.ServerActorSpawnerSubsystemClass = UServerInstancedActorsSpawnerSubsystem::StaticClass();
	DefaultConfig.ClientActorSpawnerSubsystemClass = UClientInstancedActorsSpawnerSubsystem::StaticClass();
	DefaultConfig.InstancedActorsSubsystemClass = UInstancedActorsSubsystem::StaticClass();
	DefaultConfig.StationaryVisualizationTraitClass = UInstancedActorsVisualizationTrait::StaticClass();

	CompiledActiveConfig = DefaultConfig;
}

void UInstancedActorsProjectSettings::PostInitProperties()
{
	Super::PostInitProperties();
	CompiledActiveConfig = DefaultConfig;
}

#if WITH_EDITOR
void UInstancedActorsProjectSettings::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	if (ClassConfigOverrides.IsEmpty())
	{
		CompiledActiveConfig = DefaultConfig;
	}
	else
	{
		CompileSettings();
	}

	Super::PostEditChangeProperty(PropertyChangedEvent);
}
#endif

TSubclassOf<UMassActorSpawnerSubsystem> UInstancedActorsProjectSettings::GetServerActorSpawnerSubsystemClass() const 
{ 
	return CompiledActiveConfig.ServerActorSpawnerSubsystemClass;
}

TSubclassOf<UMassActorSpawnerSubsystem> UInstancedActorsProjectSettings::GetClientActorSpawnerSubsystemClass() const 
{
	return CompiledActiveConfig.ClientActorSpawnerSubsystemClass;
}

TSubclassOf<UInstancedActorsSubsystem> UInstancedActorsProjectSettings::GetInstancedActorsSubsystemClass() const 
{
	return CompiledActiveConfig.InstancedActorsSubsystemClass;
}

TSubclassOf<UMassStationaryDistanceVisualizationTrait> UInstancedActorsProjectSettings::GetStationaryVisualizationTraitClass() const
{
	return CompiledActiveConfig.StationaryVisualizationTraitClass;
}

void UInstancedActorsProjectSettings::RegisterConfigOverride(UObject& Owner, const FInstancedActorsConfig& Config)
{
	FClassConfigOverrideEntry* Entry = ClassConfigOverrides.FindByPredicate([OwnerKey = &Owner](const FClassConfigOverrideEntry& Entry)
		{ 
			return Entry.Owner == OwnerKey;
		});
	if (Entry)
	{
		Entry->ConfigOverride = Config;
	}
	else
	{
		ClassConfigOverrides.Add({ &Owner, Config });
	}
	CompileSettings();
}

void UInstancedActorsProjectSettings::UnregisterConfigOverride(UObject& Owner)
{
	const int32 NumRemoved = ClassConfigOverrides.RemoveAll([OwnerKey = &Owner](const FClassConfigOverrideEntry& Entry)
		{
			return Entry.Owner == OwnerKey;
		});
	
	if (NumRemoved)
	{
		CompileSettings();
	}
}

/** 
 * A helper macro for applying overrides to the specified Property. Note that we iterate starting from the latest
 * override and quit as soon as a valid override is found.
 */
#define APPLY_OVERRIDE(Config, Property) \
	for (int32 OverrideIndex = ClassConfigOverrides.Num() - 1; OverrideIndex >=0; --OverrideIndex) \
	{ \
		const FClassConfigOverrideEntry& Entry = ClassConfigOverrides[OverrideIndex]; \
		if (Entry.ConfigOverride.Property) \
		{ \
			Config.Property = Entry.ConfigOverride.Property; \
			break; \
		} \
	} \

void UInstancedActorsProjectSettings::CompileSettings()
{
	CompiledActiveConfig = DefaultConfig;

	APPLY_OVERRIDE(CompiledActiveConfig, ServerActorSpawnerSubsystemClass);
	APPLY_OVERRIDE(CompiledActiveConfig, ClientActorSpawnerSubsystemClass);
	APPLY_OVERRIDE(CompiledActiveConfig, InstancedActorsSubsystemClass);
	APPLY_OVERRIDE(CompiledActiveConfig, StationaryVisualizationTraitClass);

	OnSettingsUpdated.Broadcast();
}

#undef APPLY_OVERRIDE
