// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsSettingsTypes.h"
#include "HAL/IConsoleManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsSettingsTypes)


namespace UE::InstancedActors::CVars
{
	int32 ViewDistanceQuality = -1;
	FAutoConsoleVariableRef CVarViewDistanceQuality(
		TEXT("IA.ViewDistanceQuality"),
		ViewDistanceQuality,
		TEXT("Specifies which quality setting to use for data in InstancedActorSettingsQualityLevel (0=Low, 2 = High)"),
		ECVF_Default | ECVF_Preview);

	float GlobalLODDistanceScale = 1.0f;
	FAutoConsoleVariableRef CVarGlobalLODDistanceScale(
		TEXT("IA.GlobalLODDistanceScale"),
		GlobalLODDistanceScale,
		TEXT("Global scale applied on LODDistanceScale"),
		ECVF_Default);

	float GlobalMaxDrawDistanceScale = 1.0f;
	FAutoConsoleVariableRef CVarGlobalMaxDrawDistanceScale(
		TEXT("IA.GlobalMaxDrawDistanceScale"),
		GlobalMaxDrawDistanceScale,
		TEXT("Global scale applied on Max Draw Distance"),
		ECVF_Default);

#if !UE_BUILD_SHIPPING
	FString GlobalSettingsName;
	FAutoConsoleVariableRef CVarGlobalSettingsName(
		TEXT("IA.GlobalSettingsName"),
		GlobalSettingsName,
		TEXT("Apply global LOD params only on IAMs that contain this string"),
		ECVF_Default);
#endif
} // namespace UE::InstancedActors::CVars

//-----------------------------------------------------------------------------
// FInstancedActorsSettings
//-----------------------------------------------------------------------------
template <typename T>
T GetCurrentValue(int32 ViewDistanceQuality, const TArray<T>& Values, T DefaultValue)
{
	if (Values.IsValidIndex(ViewDistanceQuality))
	{
		return Values[ViewDistanceQuality];
	}

	// Some platforms will not specify the ViewDistanceQuality parameter. Choose highest value which should be the editor one
	return (Values.Num() == 0) ? DefaultValue : Values.Last();
}

void FInstancedActorsSettings::ComputeLODDistanceData(double& OutMaxInstanceDistance, double& MaxDrawDistanceScale, float& OutLODDistanceScale) const
{
	OutMaxInstanceDistance = GetCurrentValue<double>(UE::InstancedActors::CVars::ViewDistanceQuality, MaxInstanceDistances, DefaultMaxInstanceDistance);
	OutLODDistanceScale = GetCurrentValue<float>(UE::InstancedActors::CVars::ViewDistanceQuality, LODDistanceScales, 1.0f);
	MaxDrawDistanceScale = 1.0;

#if !UE_BUILD_SHIPPING
	bool bApplyGlobalLODParams = true;
	if (!UE::InstancedActors::CVars::GlobalSettingsName.IsEmpty())
	{
		bApplyGlobalLODParams = false;
		for (int32 SettingsOverrideNameIndex = 0; SettingsOverrideNameIndex < AppliedSettingsOverrides.Num(); ++SettingsOverrideNameIndex)
		{
			if (AppliedSettingsOverrides[SettingsOverrideNameIndex].ToString().Contains(*UE::InstancedActors::CVars::GlobalSettingsName))
			{
				bApplyGlobalLODParams = true;
				break;
			}
		}
	}
#else
	const bool bApplyGlobalLODParams = true;
#endif

	if (bApplyGlobalLODParams)
	{
		MaxDrawDistanceScale = UE::InstancedActors::CVars::GlobalMaxDrawDistanceScale;
		OutLODDistanceScale *= UE::InstancedActors::CVars::GlobalLODDistanceScale;
	}
}

bool FInstancedActorsSettings::GetAffectDistanceFieldLighting() const
{
	return GetCurrentValue<bool>(UE::InstancedActors::CVars::ViewDistanceQuality, AffectDistanceFieldLighting, true);
}

template<typename TSettingsType>
static inline void DisplaySettings(FStringBuilderBase & SettingsString, const TArray<TSettingsType>& SettingsValue, bool bOverridesOnly, bool bOverride_Settings, const TCHAR* SettingsName)
{
	if (!bOverridesOnly || bOverride_Settings)
	{
		SettingsString << SettingsName << TEXT(": ");

		for (int32 ui=0; ui < SettingsValue.Num(); ++ui)
		{
			SettingsString << FString::SanitizeFloat((float)SettingsValue[ui]) << TEXT(" ");
		}
	}
}

static inline void DisplaySettings(FStringBuilderBase & SettingsString, const FGameplayTagContainer& GameplayTags, bool bOverridesOnly, bool bOverride_Settings, const TCHAR* SettingsName)
{
	if (!bOverridesOnly || bOverride_Settings)
	{
		SettingsString << SettingsName << TEXT(": ");

		SettingsString << GameplayTags.ToStringSimple();
	}
}

void FInstancedActorsSettings::OverrideIfDefault(FConstStructView InOverrideSettings, const FName& OverrideSettingsName)
{
	const FInstancedActorsSettings& OverrideSettings = InOverrideSettings.Get<const FInstancedActorsSettings>();

	IASETTINGS_OVERRIDE_IF_DEFAULT(bInstancesCastShadows);
	IASETTINGS_OVERRIDE_IF_DEFAULT(MaxActorDistance);
	IASETTINGS_OVERRIDE_IF_DEFAULT(bDisableAutoDistanceCulling);
	IASETTINGS_OVERRIDE_IF_DEFAULT(MaxInstanceDistances);
	IASETTINGS_OVERRIDE_IF_DEFAULT(AffectDistanceFieldLighting);
	IASETTINGS_OVERRIDE_IF_DEFAULT(DetailedRepresentationLODDistance);
	IASETTINGS_OVERRIDE_IF_DEFAULT(ForceLowRepresentationLODDistance);
	IASETTINGS_OVERRIDE_IF_DEFAULT(WorldPositionOffsetDisableDistance);	
	IASETTINGS_OVERRIDE_IF_DEFAULT(bEjectOnActorMoved);
	IASETTINGS_OVERRIDE_IF_DEFAULT(ActorEjectionMovementThreshold);
	IASETTINGS_OVERRIDE_IF_DEFAULT(bCanEverAffectNavigation);
	IASETTINGS_OVERRIDE_IF_DEFAULT(OverrideWorldPartitionGrid);
	IASETTINGS_OVERRIDE_IF_DEFAULT(LODDistanceScales);
	IASETTINGS_OVERRIDE_IF_DEFAULT(ScaleEntityCount);
	IASETTINGS_OVERRIDE_IF_DEFAULT(ActorClass);
	IASETTINGS_OVERRIDE_IF_DEFAULT(bCanBeDamaged);
	IASETTINGS_OVERRIDE_IF_DEFAULT(bIgnoreModifierVolumes);
	IASETTINGS_OVERRIDE_IF_DEFAULT(bModifierVolumeCheckFullyEnclosed);
	IASETTINGS_OVERRIDE_IF_DEFAULT(bControlPhysicsState);
	IASETTINGS_OVERRIDE_IF_DEFAULT(GameplayTags);
	
	AppliedSettingsOverrides.Add(OverrideSettingsName);
}

FString FInstancedActorsSettings::DebugToString(bool bOverridesOnly) const
{
	TStringBuilder<0> SettingsString;
	
	IASETTINGS_SETTING_TO_STRING(bInstancesCastShadows);
	IASETTINGS_FLOAT_SETTING_TO_STRING(MaxActorDistance);
	IASETTINGS_SETTING_TO_STRING(bDisableAutoDistanceCulling);
	DisplaySettings(SettingsString, MaxInstanceDistances, bOverridesOnly, bOverride_MaxInstanceDistances, TEXT("MaxInstanceDistances"));
	DisplaySettings(SettingsString, AffectDistanceFieldLighting, bOverridesOnly, bOverride_AffectDistanceFieldLighting, TEXT("AffectDistanceFieldLighting"));
	IASETTINGS_FLOAT_SETTING_TO_STRING(DetailedRepresentationLODDistance);
	IASETTINGS_FLOAT_SETTING_TO_STRING(ForceLowRepresentationLODDistance);
	IASETTINGS_FLOAT_SETTING_TO_STRING(WorldPositionOffsetDisableDistance);
	IASETTINGS_SETTING_TO_STRING(bEjectOnActorMoved);
	IASETTINGS_FLOAT_SETTING_TO_STRING(ActorEjectionMovementThreshold);
	IASETTINGS_SETTING_TO_STRING(bCanEverAffectNavigation);
	IASETTINGS_SETTING_TO_STRING(OverrideWorldPartitionGrid);
	IASETTINGS_SETTING_TO_STRING(ScaleEntityCount);
	IASETTINGS_UOBJECT_SETTING_TO_STRING(ActorClass);
	IASETTINGS_SETTING_TO_STRING(bCanBeDamaged);
	IASETTINGS_SETTING_TO_STRING(bIgnoreModifierVolumes);
	IASETTINGS_SETTING_TO_STRING(bModifierVolumeCheckFullyEnclosed);
	IASETTINGS_SETTING_TO_STRING(bControlPhysicsState);
	DisplaySettings(SettingsString, GameplayTags, bOverridesOnly, bOverride_GameplayTags, TEXT("GameplayTags"));

	DisplaySettings(SettingsString, LODDistanceScales, bOverridesOnly, bOverride_LODDistanceScales, TEXT("LODDistanceScales"));

	if (AppliedSettingsOverrides.Num() > 0)
	{
		SettingsString << TEXT("AppliedSettings: ");
		SettingsString << FString::JoinBy(AppliedSettingsOverrides, TEXT(","), [](const FName& SettingsName) { return SettingsName.ToString(); });
	}

	return FString(SettingsString);
}
