// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsStationaryLODBatchProcessor.h"
#include "GameFramework/PlayerController.h"

#include "InstancedActorsSettingsTypes.h"
#include "InstancedActorsSettings.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"
#include "InstancedActorsDebug.h"
#include "InstancedActorsSubsystem.h"
#include "InstancedActorsCommands.h"
#include "InstancedActorsVisualizationProcessor.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "StaticMeshResources.h"

#include "MassActorSubsystem.h"
#include "MassCommands.h"
#include "MassExecutionContext.h"
#include "MassLODSubsystem.h"
#include "MassLODTypes.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationProcessor.h"
#include "MassRepresentationTypes.h"
#include "MassSignalSubsystem.h"
#include "MassStationaryISMSwitcherProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsStationaryLODBatchProcessor)


DECLARE_CYCLE_STAT(TEXT("InstancedActors LODBatchProcessor"), STAT_InstancedActorsStationaryLODBatchProcessor_Execute, STATGROUP_Mass);

namespace UE::Mass::Tweakables
{
bool bBatchedStationaryLODEnabled = true;
bool bLODBasedTicking = true;
bool bControlPhysicsState = true;
bool bUpdateLiveCullDistanceTweaking = false;
float DebugDetailedLevelDistanceOverride = 0.f;

namespace 
{
static FAutoConsoleVariableRef AnonymousCVars[] = {
	{
		TEXT("IA.BatchedStationaryLODEnabled"),
		bBatchedStationaryLODEnabled,
		TEXT(""), ECVF_Cheat
	},
	{
		TEXT("IA.LODBasedTicking"),
		bLODBasedTicking,
		TEXT(""), ECVF_Cheat
	},
	{
		TEXT("IA.LODDrivenPhysicsState"),
		bControlPhysicsState,
		TEXT(""), ECVF_Cheat
	},
	{
		TEXT("IA.UpdateLiveCullDistanceTweaking"),
		bUpdateLiveCullDistanceTweaking,
		TEXT(""), ECVF_Cheat
	},
	{
		TEXT("IA.debug.DetailedLevelDistanceOverride"),
		DebugDetailedLevelDistanceOverride,
		TEXT(""), ECVF_Cheat
	}
};
}

} // UE::Mass::Tweakables

namespace UE::InstancedActors
{
	bool EnablePhysicForVisualization(uint8 /*VisualizationIndex*/, const FInstancedActorsVisualizationInfo& Visualization)
	{
		for (const TObjectPtr<UInstancedStaticMeshComponent>& ISMComponent : Visualization.ISMComponents)
		{
			if (ISMComponent)
			{
				if (ISMComponent->IsRegistered())
				{
					ISMComponent->CreatePhysicsState(/*bAllowDeferral=*/true);
				}
				else
				{
					UE_LOGF(LogInstancedActors, Error, "Failed to call CreatePhysicsState() on component '%ls', because component is not registered.", *GetFullNameSafe(ISMComponent));
				}
			}
		}
		return true;
	}

	bool DisablePhysicForVisualization(uint8 /*VisualizationIndex*/, const FInstancedActorsVisualizationInfo& Visualization)
	{
		for (const TObjectPtr<UInstancedStaticMeshComponent>& ISMComponent : Visualization.ISMComponents)
		{
			if (ISMComponent)
			{
				ISMComponent->DestroyPhysicsState();
			}
		}
		return true;
	}
}

//-----------------------------------------------------------------------------
// UInstancedActorsStationaryLODBatchProcessor
//-----------------------------------------------------------------------------
UInstancedActorsStationaryLODBatchProcessor::UInstancedActorsStationaryLODBatchProcessor()
	: LODChangingEntityQuery(*this)
	, DirtyVisualizationEntityQuery(*this)
{
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Representation);
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LOD);
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LODCollector);

	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;

     // Direct ISM component modifications are not thread-safe and must run on the game thread.
	bRequiresGameThreadExecution = true;

	DelayPerBulkLOD[(int)EInstancedActorsBulkLOD::Detailed] = 5.0;
	DelayPerBulkLOD[(int)EInstancedActorsBulkLOD::Medium] = 1.0;
	DelayPerBulkLOD[(int)EInstancedActorsBulkLOD::Low] = 2.5;
	DelayPerBulkLOD[(int)EInstancedActorsBulkLOD::Off] = 10.0;
}

void UInstancedActorsStationaryLODBatchProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	ProcessorRequirements.AddSubsystemRequirement<UMassLODSubsystem>(EMassFragmentAccess::ReadOnly);
	if (HasAnyFlags(RF_ClassDefaultObject) == false)
	{
		if (TSubclassOf<UInstancedActorsSubsystem> InstancedActorsSubsystemClass = GET_INSTANCEDACTORS_CONFIG_VALUE(GetInstancedActorsSubsystemClass()))
		{
			ProcessorRequirements.AddSubsystemRequirement(InstancedActorsSubsystemClass, EMassFragmentAccess::ReadWrite, EntityManager);
		}
	}
	LODChangingEntityQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadWrite);
	
	// required by the UMassRepresentationProcessor::UpdateRepresentation call
	// the commented-out requirements are here for the reference, already added above.
	LODChangingEntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	LODChangingEntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	//LODChangingEntityQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadOnly);
	LODChangingEntityQuery.AddRequirement<FMassActorFragment>(EMassFragmentAccess::ReadWrite);
	LODChangingEntityQuery.AddConstSharedRequirement<FMassRepresentationParameters>();
	LODChangingEntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	LODChangingEntityQuery.AddSubsystemRequirement<UMassActorSubsystem>(EMassFragmentAccess::ReadWrite);

	// required by the UMassStationaryISMSwitcherProcessor::ProcessContext call
	// the commented-out requirements are here for the reference, already added above.
	//EntityQuery.AddRequirement<FMassRepresentationLODFragment>(EMassFragmentAccess::ReadOnly);
	//EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	//EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	//EntityQuery.AddConstSharedRequirement<FMassRepresentationParameters>();
	//EntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	LODChangingEntityQuery.AddTagRequirement<FMassStaticRepresentationTag>(EMassFragmentPresence::All);
	LODChangingEntityQuery.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);

	DirtyVisualizationEntityQuery = LODChangingEntityQuery;
	DirtyVisualizationEntityQuery.AddRequirement<FInstancedActorsFragment>(EMassFragmentAccess::ReadOnly);
}

void UInstancedActorsStationaryLODBatchProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	SCOPE_CYCLE_COUNTER(STAT_InstancedActorsStationaryLODBatchProcessor_Execute);

	// some of the code below assumes EInstancedActorsBulkLOD::Detailed == 0, we need to verify that's the case. If not the code below needs updating.
	static_assert((uint8)EInstancedActorsBulkLOD::Detailed == 0, "Code below relies on the assumptions. Needs to be updated if the assumption is broken");

	if (UE::Mass::Tweakables::bBatchedStationaryLODEnabled == false)
	{
		return;
	}

	TSubclassOf<UInstancedActorsSubsystem> InstancedActorsSubsystemClass = GET_INSTANCEDACTORS_CONFIG_VALUE(GetInstancedActorsSubsystemClass());
	if (!ensureMsgf(InstancedActorsSubsystemClass, TEXT("Misconfigured UInstancedActorsSubsystem subclass")))
	{
		return;
	}
	//@todo: revisit this temporary fix. This was done because UGameFeatureAction_ConfigureInstancedActors::OnGameFeatureActivating() may happen after UInstancedActorsStationaryLODBatchProcessor::ConfigureQueries()
	//UInstancedActorsSubsystem* InstancedActorSubsystem = Context.GetMutableSubsystem<UInstancedActorsSubsystem>(InstancedActorsSubsystemClass);
	UInstancedActorsSubsystem* InstancedActorSubsystem = UE::InstancedActors::Utils::GetInstancedActorsSubsystem(*Context.GetWorld());
	if (!ensureMsgf(InstancedActorSubsystem, TEXT("UInstancedActorsSubsystem is missing, this is unexpected")))
	{
		return;
	}

	const UMassLODSubsystem& LODSubsystem = Context.GetSubsystemChecked<UMassLODSubsystem>();
	// note we're copying the array on purpose since we intend to use parallel-for here (eventually)
	TArray<FViewerInfo> Viewers = LODSubsystem.GetViewers();

	// we don't care about streaming-sources
	for (int32 ViewerIndex = Viewers.Num() - 1; ViewerIndex >= 0; --ViewerIndex)
	{
		if (Viewers[ViewerIndex].StreamingSourceName.IsNone() == false)
		{
			Viewers.RemoveAtSwap(ViewerIndex, EAllowShrinking::No);
		}
		else if (Viewers[ViewerIndex].Location.IsNearlyZero() == true)
		{
			// we can end up with "nearly zero" location in two cases:
			// 1. player's pawn or camera is actually at that location
			// 2. the player hasn't really started yet so there's no pawn and the camera is in its inital location
			// We need to filter out the latter. 
			// Note that we rely on UMassSubsystem::bUsePlayerPawnLocationInsteadOfCamera being true here. Without it there's no 
			// reliable way to differentiate the cases - this property is checked in UInstancedActorsSubsystem::Initialize
			if (APlayerController* ViewerAsPlayerController = Viewers[ViewerIndex].GetPlayerController())
			{
				if (ViewerAsPlayerController->GetPawn() == nullptr)
				{
					// no pawn so this is definitely case number 2.
					Viewers.RemoveAtSwap(ViewerIndex, EAllowShrinking::No);
				}
			}
		}
	}

	if (Viewers.Num())
	{
		checkSlow(LODSubsystem.GetWorld());
		const double CurrentTime = LODSubsystem.GetWorld()->TimeSeconds;

		static const auto ICVarStaticMeshLODDistanceScale = IConsoleManager::Get().FindConsoleVariable(TEXT("r.StaticMeshLODDistanceScale"));
		const float StaticMeshLODDistanceScale = ICVarStaticMeshLODDistanceScale->GetFloat();

		auto ExecutionFunction = [Viewers = MakeArrayView((const FViewerInfo*)&Viewers[0], Viewers.Num()), &EntityManager, &Context
			, LODChangingEntityQuery = &LODChangingEntityQuery, StaticMeshLODDistanceScale, CurrentTime
			, DelayPerBulkLOD = MakeArrayView((const double*)&DelayPerBulkLOD[0], (int)EInstancedActorsBulkLOD::MAX)]
			(FInstancedActorsDataSharedFragment& ManagerSharedFragment) -> double
			{
				double NextTickTime = CurrentTime + DelayPerBulkLOD[(int)EInstancedActorsBulkLOD::Off];

				UInstancedActorsData* InstanceData = ManagerSharedFragment.InstanceData.Get();
				if (UE::InstancedActors::CVars::bFlagSharedFragmentInsteadOfNullify)
				{
					if (ManagerSharedFragment.bInstancedActorsDataDespawnedEntities)
					{
						InstanceData = nullptr;
					}
				}

				if (InstanceData != nullptr)
				{
					const FInstancedActorsSettings& Settings = InstanceData->GetSettings<const FInstancedActorsSettings>();
					
					const FVector::FReal ForcedDetailedLevelDistanceSquared = FMath::Square(
#if WITH_INSTANCEDACTORS_DEBUG
						UE::Mass::Tweakables::DebugDetailedLevelDistanceOverride ? FVector::FReal(UE::Mass::Tweakables::DebugDetailedLevelDistanceOverride) :
#endif
						Settings.DetailedRepresentationLODDistance
					);

					// Calculates distance sqr from the viewer to the bounds of the InstancedActorManager who owns the FInstancedActorsDataSharedFragment
					const FBox WorldSpaceBounds = InstanceData->Bounds.TransformBy(InstanceData->GetManagerChecked().GetActorTransform());
					FVector::FReal DistanceSquared = TNumericLimits<FVector::FReal>::Max();

					for (const FViewerInfo& ViewerInfo : Viewers)
					{
						DistanceSquared = FMath::Min(DistanceSquared, ComputeSquaredDistanceFromBoxToPoint(WorldSpaceBounds.Min, WorldSpaceBounds.Max, ViewerInfo.Location));
						if (DistanceSquared < ForcedDetailedLevelDistanceSquared)
						{
							// if it's inside the "inner circle" we don't need to continue calculating the distance.
							break;
						}
					}

#if !(UE_BUILD_SHIPPING || UE_BUILD_TEST)
					// Only update cull distances when tweaking is enabled for runtime profiling & iteration
					if (UE::Mass::Tweakables::bUpdateLiveCullDistanceTweaking)
					{
						InstanceData->UpdateCullDistance();
					}
#endif

					// Calculates LOD for a given FInstancedActorsDataSharedFragment based on the distance from the viewer to its owner's bounds
					// NOTE (1): It's called bulk LOD because we're only comparing the viewer to the InstancedActorManager, and not to a specific instance inside it
					// NOTE (2): We're caching the scaled squared draw distance to the lowest LOD because the cvar could change
					const float ScaledForceLowLODDrawDistance = InstanceData->LowLODDrawDistance / StaticMeshLODDistanceScale;
					EInstancedActorsBulkLOD NewBulkLOD = EInstancedActorsBulkLOD::Off;
					if (DistanceSquared < ForcedDetailedLevelDistanceSquared)
					{
						NewBulkLOD = EInstancedActorsBulkLOD::Detailed;
					}
					else if (DistanceSquared < FMath::Square(ScaledForceLowLODDrawDistance))
					{
						NewBulkLOD = EInstancedActorsBulkLOD::Medium;
					}
					else if (DistanceSquared < FMath::Square(InstanceData->MaxDrawDistance) || (InstanceData->MaxDrawDistance == 0.0f))
					{
						NewBulkLOD = EInstancedActorsBulkLOD::Low;
					}

					check(NewBulkLOD != EInstancedActorsBulkLOD::MAX);
					// Updates the time at which the FInstancedActorsDataSharedFragment will tick depending on its bulk LOD value
					NextTickTime = CurrentTime + (DelayPerBulkLOD[(int)NewBulkLOD] * 0.95 + FMath::FRand() * 0.1);

					if (const bool bHasBulkLODChanged = (ManagerSharedFragment.BulkLOD != NewBulkLOD))
					{
						// Decrements stats with current state, and then increments stats with the new one
						{
							AInstancedActorsManager::UpdateInstanceStats(InstanceData->NumInstances, ManagerSharedFragment.BulkLOD, false);
							ManagerSharedFragment.BulkLOD = NewBulkLOD;
							AInstancedActorsManager::UpdateInstanceStats(InstanceData->NumInstances, ManagerSharedFragment.BulkLOD, true);
						}
						// Toggles physics state for the IA's ISM depending on the new bulk LOD value.
						// If enabled = physics on, else = physics off.				
						if (UE::Mass::Tweakables::bControlPhysicsState && Settings.bControlPhysicsState)
						{
							if (NewBulkLOD == EInstancedActorsBulkLOD::Detailed)
							{
								InstanceData->ForEachVisualization(&UE::InstancedActors::EnablePhysicForVisualization);
							}
							else
							{
								InstanceData->ForEachVisualization(UE::InstancedActors::DisablePhysicForVisualization);
							}
						}
						{
							// Toggles visibility for the IA's ISM depending on the new bulk LOD value.
							// If enabled = use default visibility (probably on), else = physics off.
							if (NewBulkLOD != EInstancedActorsBulkLOD::Off)
							{
								const bool bForcedLowLOD = NewBulkLOD == EInstancedActorsBulkLOD::Low;
								InstanceData->ForEachVisualization([&bForcedLowLOD](uint8 VisualizationIndex, const FInstancedActorsVisualizationInfo& Visualization)
								{
									for (int32 ISMComponentIndex = 0; ISMComponentIndex < Visualization.ISMComponents.Num(); ++ISMComponentIndex)
									{
										check(Visualization.VisualizationDesc.ISMComponentDescriptors.IsValidIndex(ISMComponentIndex));
										const FISMComponentDescriptor& ISMComponentDescriptor = Visualization.VisualizationDesc.ISMComponentDescriptors[ISMComponentIndex];
										const TObjectPtr<UInstancedStaticMeshComponent>& ISMComponent = Visualization.ISMComponents[ISMComponentIndex];

										ISMComponent->SetVisibility(ISMComponentDescriptor.bVisible); // Restore default visibility state
										ISMComponent->SetForcedLodModel(bForcedLowLOD ? 8 : 0); // 0 means forced LOD disabled, 8 means lowest because it's clamped		
									}
									return true;
								});
							}
							else
							{
								InstanceData->ForEachVisualization([](uint8 VisualizationIndex, const FInstancedActorsVisualizationInfo& Visualization)
								{
									for (const TObjectPtr<UInstancedStaticMeshComponent>& ISMComponent : Visualization.ISMComponents)
									{
										ISMComponent->SetVisibility(false);
									}
									return true;
								});
							}
						}
						// Toggles MassProcessors on/off depending on the bulk LOD, by pushing or removing tags that are used by those processor's queries.
						// NOTE: Forcibly updates the mass LOD to off or low when bulk LOD is smaller than Detailed
						if (ManagerSharedFragment.BulkLOD == EInstancedActorsBulkLOD::Detailed && InstanceData->CanHydrate())
						{
							EntityManager.Defer().PushCommand<UE::InstancedActors::FEnableDetailedLODCommand>(InstanceData, EInstancedActorsBulkLOD::Detailed);
						}
						else
						{
							// Force given LOD for all the hosted entities 
							EMassLOD::Type NewLOD = EMassLOD::Off;
							switch (ManagerSharedFragment.BulkLOD)
							{
							case EInstancedActorsBulkLOD::Detailed:
								ensureMsgf(InstanceData->CanHydrate() == false, TEXT("This case is only valid for non-hydrating instance, broken for %s"), *GetNameSafe(InstanceData->ActorClass));
								NewLOD = EMassLOD::Low;
								break;
							case EInstancedActorsBulkLOD::Medium: // right now falling through since we don't have a medium-level visualization
							case EInstancedActorsBulkLOD::Low:
								NewLOD = EMassLOD::Low;
								break;
							default:
								NewLOD = EMassLOD::Off;
								break;
							}

							// Grabs entity collections from the entities stored by the fragment we're processing, so that we can process them as chunks
							TArray<FMassArchetypeEntityCollection> EntityCollections;
							UE::Mass::Utils::CreateEntityCollections(EntityManager, InstanceData->Entities, FMassArchetypeEntityCollection::NoDuplicates, EntityCollections);

							LODChangingEntityQuery->ForEachEntityChunkInCollections(EntityCollections, Context, [NewLOD](FMassExecutionContext& Context)
								{
									const TArrayView<FMassRepresentationLODFragment> RepresentationLODFragments = Context.GetMutableFragmentView<FMassRepresentationLODFragment>();
									for (FMassRepresentationLODFragment& LODFragment : RepresentationLODFragments)
									{
										LODFragment.LOD = NewLOD;
									}

									FMassRepresentationUpdateParams Params;
									Params.bTestCollisionAvailibilityForActorVisualization = false;
									UMassRepresentationProcessor::UpdateRepresentation(Context, Params);
									UMassStationaryISMSwitcherProcessor::ProcessContext(Context);
								});

							// Removes a bunch of tags from all mass entities that belong to an InstancedActorsData, so that we don't spend MassProcessor time on them
							EntityManager.Defer().PushCommand<UE::InstancedActors::FEnableBatchLODCommand>(InstanceData, ManagerSharedFragment.BulkLOD);
						}
					}
				}

				return NextTickTime;
			};

		TArray<UInstancedActorsSubsystem::FNextTickSharedFragment>& SortedSharedFragments = InstancedActorSubsystem->GetTickableSharedFragments();
		if (SortedSharedFragments.Num() > 0)
		{
			while (SortedSharedFragments.HeapTop().NextTickTime < CurrentTime)
			{
				UInstancedActorsSubsystem::FNextTickSharedFragment WrappedSharedFragment;
				SortedSharedFragments.HeapPop(WrappedSharedFragment, EAllowShrinking::No);
				FInstancedActorsDataSharedFragment& ManagerSharedFragment = WrappedSharedFragment.SharedStruct.Get<FInstancedActorsDataSharedFragment>();
				
				ManagerSharedFragment.LastTickTime = CurrentTime;
				WrappedSharedFragment.NextTickTime = ExecutionFunction(ManagerSharedFragment);
				SortedSharedFragments.HeapPush(MoveTemp(WrappedSharedFragment));
			}
		}

		// Consume all pending explicitly dirtied instances to process
		// NOTE: Those instances are dirtied whenever an InstancedActor is hydrated/dehydrated (check UInstancedActorsData::SetReplicatedActor)
		TArray<FInstancedActorsInstanceHandle> DirtyRepresentationInstances;
		InstancedActorSubsystem->PopAllDirtyRepresentationInstances(DirtyRepresentationInstances);

		if (DirtyRepresentationInstances.Num())
		{
			const bool bInvalidInstancedAllowed = (InstancedActorSubsystem->GetWorld() == nullptr) || (InstancedActorSubsystem->GetWorld()->GetNetMode() == NM_Client);

			// Collect mass entities from instance handles into entity collections for processing
			// UE::Mass::Utils::CreateEntityCollections but from TArray<FInstancedActorsInstanceHandle>, retrieving instance entities as we go
			TMap<const FMassArchetypeHandle, TArray<FMassEntityHandle>> DirtyEntitiesByArchetype;
			for (const FInstancedActorsInstanceHandle& DirtyRepresentationInstance : DirtyRepresentationInstances)
			{
				// Note that it's possible for DirtyRepresentationInstances to contain indices to entities that just have 
				// been destroyed (however, we only expect it to happen only on clients, of if we're somewhere in the process of destroying the world). 
				UE_CLOGF(!(DirtyRepresentationInstance.IsValid() || bInvalidInstancedAllowed), LogInstancedActors, Warning
					, "We only expect invalid instance handles on Client or when the InstancedActorSubsystem no longer has a valid outer UWorld.");

				if (DirtyRepresentationInstance.IsValid() 
					// only the entities that are not "Detailed" require update, so we're filtering out accordingly
					&& DirtyRepresentationInstance.GetInstanceActorDataChecked().GetBulkLOD() > EInstancedActorsBulkLOD::Detailed)
				{
					FMassEntityHandle DirtyEntity = DirtyRepresentationInstance.GetInstanceActorDataChecked().GetEntity(DirtyRepresentationInstance.GetInstanceIndex());
					if (EntityManager.IsEntityValid(DirtyEntity))
					{
						FMassArchetypeHandle EntityArchetype = EntityManager.GetArchetypeForEntityUnsafe(DirtyEntity);
						TArray<FMassEntityHandle>& DirtyEntities = DirtyEntitiesByArchetype.FindOrAdd(EntityArchetype);
						DirtyEntities.Add(DirtyEntity);
					}
				}
			}

			if (DirtyEntitiesByArchetype.Num())
			{
				// Converts collected mass entities to collections, which we'll then process afterwards as entity chunks
				TArray<FMassArchetypeEntityCollection> DirtyEntityCollections;
				for (TPair<const FMassArchetypeHandle, TArray<FMassEntityHandle>>& Pair : DirtyEntitiesByArchetype)
				{
					DirtyEntityCollections.Add(FMassArchetypeEntityCollection(Pair.Key, Pair.Value, FMassArchetypeEntityCollection::EDuplicatesHandling::FoldDuplicates));
				}

				// Ensure that detailed representation update occurs for explicitly dirtied instanced actor entities with non-detailed BulkLOD 
				DirtyVisualizationEntityQuery.ForEachEntityChunkInCollections(DirtyEntityCollections, Context, [](FMassExecutionContext& Context)
					{
						// It's possible that we've only just switched to Non-Detailed this frame, the tag removal to prevent regular processing wouldn't have
						// occurred yet and we would have performed a representation update this frame already.
						if (!Context.DoesArchetypeHaveTag<FInstancedActorsVisualizationProcessorTag>())
						{
							FMassRepresentationUpdateParams Params;
							Params.bTestCollisionAvailibilityForActorVisualization = false;
							UMassRepresentationProcessor::UpdateRepresentation(Context, Params);
						}
						if (!Context.DoesArchetypeHaveTag<FMassStationaryISMSwitcherProcessorTag>())
						{
							UMassStationaryISMSwitcherProcessor::ProcessContext(Context);
						}
					});
			}
		}
	}
}
