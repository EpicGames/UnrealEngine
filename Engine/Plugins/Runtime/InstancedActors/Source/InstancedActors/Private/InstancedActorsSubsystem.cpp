// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsSubsystem.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsModifierVolume.h"
#include "InstancedActorsModifierVolumeComponent.h"
#include "InstancedActorsDebug.h"
#include "InstancedActorsData.h"
#include "InstancedActorsSettingsTypes.h"
#include "InstancedActorsSettings.h"
#include "ActorPartition/ActorPartitionSubsystem.h"
#include "Algo/Find.h"
#include "Algo/IndexOf.h"
#include "DataRegistry.h"
#include "DataRegistrySubsystem.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "EngineUtils.h"
#include "MassEntityTypes.h"
#include "MassEntitySubsystem.h"
#include "Misc/ArchiveMD5.h"
#include "Misc/ReverseIterate.h"
#include "VisualLogger/VisualLogger.h"

#if WITH_EDITOR
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"
#include "UObject/UObjectIterator.h"
#endif // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsSubsystem)


namespace InstancedActorsCVars
{
	float MaxDeferSpawnEntitiesTimePerTick = 0.0015f;
	FAutoConsoleVariableRef CVarMaxDeferSpawnEntitiesTimePerTick(
		TEXT("IA.DeferSpawnEntities.MaxTimePerTick"),
		MaxDeferSpawnEntitiesTimePerTick,
		TEXT("When IA.DeferSpawnEntities is enabled, the max time in seconds to spend per frame executing deferred entity spawning.")
		TEXT("After this time, remaining requests will be left for subsequent frames. INFINITY = Unbounded deferred spawning."),
		ECVF_Default);

	float ManagerHashGridSize = 500.0f;
	FAutoConsoleVariableRef CVarManagerHashGridSize(
		TEXT("IA.ManagerHashGridSize"),
		ManagerHashGridSize,
		TEXT("The THierarchicalHashGrid2D cell size for managers"),
		ECVF_Default);

	float ModifierVolumeHashGridSize = 500.0f;
	FAutoConsoleVariableRef CVarModifierVolumeHashGridSize(
		TEXT("IA.ModifierVolumeHashGridSize"),
		ModifierVolumeHashGridSize,
		TEXT("The THierarchicalHashGrid2D cell size for modifier volumes"),
		ECVF_Default);

	int32 RuntimeEnforceActorClassSettingsPresence = 0;
	FAutoConsoleVariableRef CVarRuntimeEnforceActorClassSettingsPresence(
		TEXT("IA.RuntimeEnforceActorClassSettingsPresence"),
		RuntimeEnforceActorClassSettingsPresence,
		TEXT("The error severity to use when no FInstancedActorsClassSettingsBase are found for a given ActorClass (or any of it's superclasses) in the ActorClassSettingsRegistry")
		TEXT("at runtime. Useful for ensuring unknown / unoptimized actor classes aren't being unexpectedly instanced.")
		TEXT("0 = No error, ActorClass's are not required to be be present in ActorClassSettingsRegistry at all.")
		TEXT("1 = Log an error, continue to instance ActorClass regardless.")
		TEXT("2 = Ensure (log stack trace and break debugger)."),
		ECVF_Default);

	int32 EditorEnforceActorClassSettingsPresence = 1;
	FAutoConsoleVariableRef CVarEditorEnforceActorClassSettingsPresence(
		TEXT("IA.EditorEnforceActorClassSettingsPresence"),
		EditorEnforceActorClassSettingsPresence,
		TEXT("The error severity to use when no FInstancedActorsClassSettingsBase are found for a given ActorClass (or any of it's superclasses) in the ActorClassSettingsRegistry")
		TEXT("when instancing actors in the editor. Useful for ensuring unknown / unoptimized actor classes aren't being unexpectedly instanced.")
		TEXT("0 = No error, ActorClass's are not required to be be present in ActorClassSettingsRegistry at all.")
		TEXT("1 = Log a message log warning, continue to instance ActorClass regardless.")
		TEXT("2 = Log a message log error, skip instancing ActorClass.")
		TEXT("3 = Ensure (log stack trace and break debugger), log a message log error, skip instancing ActorClass."),
		ECVF_Default);

#if WITH_EDITOR
	static TAutoConsoleVariable<int32> CVarRefreshSettings(
		TEXT("IA.RefreshSettings"),
		0,
		TEXT("Refresh Settings"),
		ECVF_Default
	);
#endif
}

//-----------------------------------------------------------------------------
// UInstancedActorsSubsystem
//-----------------------------------------------------------------------------
UInstancedActorsSubsystem::UInstancedActorsSubsystem()
{
	SettingsType = FInstancedActorsSettings::StaticStruct();
	InstancedActorsManagerClass = AInstancedActorsManager::StaticClass();
}

UInstancedActorsSubsystem* UInstancedActorsSubsystem::Get(UObject* WorldContextObject)
{
	if (UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull))
	{
		return UE::InstancedActors::Utils::GetInstancedActorsSubsystem(*World);
	}

	return nullptr;
}

UInstancedActorsSubsystem& UInstancedActorsSubsystem::GetChecked(UObject* WorldContextObject)
{
	UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::Assert);
	check(World);
	UInstancedActorsSubsystem* Subsystem = UE::InstancedActors::Utils::GetInstancedActorsSubsystem(*World);
	check(Subsystem);
	return *Subsystem;
}

bool UInstancedActorsSubsystem::ShouldCreateSubsystem(UObject* Outer) const
{
	if (Super::ShouldCreateSubsystem(Outer) && Outer)
	{
		if (UWorld* World = Outer->GetWorld())
		{
#if WITH_EDITOR
			// we don't want to create subsystems for Editor worlds while PIE is active
			// This wouldn't happen in normal world lifecycle, but can happen if FSubsystemCollectionBase::ActivateExternalSubsystem
			// is used (it adds an instance of a given subsystem class to ALL worlds) - for example by GameFeatureActions.
			if (GEditor && GEditor->IsPlayingSessionInEditor() && World->WorldType == EWorldType::Editor)
			{
				return false;
			}
#endif // WITH_EDITOR

			// we only ever want to have a single instance of this subsystem. Attempting to add multiple
			// instances can be a result of subsystem adding game feature actions.
			return World->GetSubsystemBase(GetClass()) == nullptr;
		}
	}
	return false;
}

void UInstancedActorsSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedActorsSubsystem Initialize);

	ActorPartitionSubsystem = Collection.InitializeDependency<UActorPartitionSubsystem>();

	DataRegistrySubsystem = UDataRegistrySubsystem::Get();
	check(IsValid(DataRegistrySubsystem));

	ProjectSettings = GetDefault<UInstancedActorsProjectSettings>();
	check(IsValid(ProjectSettings));

	ManagersHashGrid = FManagersHashGridType(InstancedActorsCVars::ManagerHashGridSize);
	ModifierVolumesHashGrid = FModifierVolumesHashGridType(InstancedActorsCVars::ModifierVolumeHashGridSize);

#if WITH_EDITOR
	InstancedActorsCVars::CVarRefreshSettings.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate::CreateUObject(this, &UInstancedActorsSubsystem::HandleRefreshSettings));
#endif

	UWorld* World = GetWorld();
	check(World);

	UMassEntitySubsystem* EntitySubsystem = Collection.InitializeDependency<UMassEntitySubsystem>();
	check(EntitySubsystem);
	EntityManager = EntitySubsystem->GetMutableEntityManager().AsShared();

	// As playlist GFP's are initialized after main map load, we account for latent subsystem creation here by registering any existing
	// AInstancedActorsModifierVolume's and AInstancedActorsManager's that may already have loaded before subsystem creation.

	// Collect existing modifier volumes, calling AInstancedActorsModifierVolume::OnAddedToSubsystem to inform them of latent addition 
	// to this subsystem
	//
	// Note: Modifiers *must* be collected before managers, to ensure managers can then retrieve these modifiers in 
	// AInstancedActorsManager::OnAddedToSubsystem, providing managers an opportunity to run optimized pre-entity-spawning modifiers.
	for (TActorIterator<AInstancedActorsModifierVolume> ModifierVolumeIt(World); ModifierVolumeIt; ++ModifierVolumeIt)
	{
		AInstancedActorsModifierVolume* ModifierVolume = *ModifierVolumeIt;
		check(ModifierVolumeIt);

		AddModifierVolume(*ModifierVolume->GetModifierVolumeComponent());
	}

	// Collect existing managers, calling AInstancedActorsManager::OnAddedToSubsystem to inform them of latent addition to this subsystem
	for (TActorIterator<AInstancedActorsManager> ManagerIt(World); ManagerIt; ++ManagerIt)
	{
		AInstancedActorsManager* Manager = *ManagerIt;
		check(Manager);
		// we only case about managers that have already begun play and missed their chance to registered in their BeginPlay
		if (Manager->HasActorBegunPlay())
		{
			AddManager(*Manager);
		}
	}

	// InstancedActors rely on Mass LOD subsystem, and we expect some specific configuration to work properly
	const UMassLODSubsystem* LODSubsystem = Collection.InitializeDependency<UMassLODSubsystem>();
	if (ensureMsgf(LODSubsystem, TEXT("InstancedActors require MassLODSubsystem's existence to function properly")))
	{
		UE_CLOGF(LODSubsystem->IsUsingPlayerPawnLocationInsteadOfCamera() == false
			, LogInstancedActors, Log, "Using Player's camera location for instanced actors LOD calculations - this can skew the LOD calculations in non-FPP games.");
	}

	// Note that we're using GetClass() rather than StaticClass() to work as expected for child-classes as well.
	// Child class can always override the traits registered this way.
	UE::Mass::Subsystems::RegisterSubsystemType(EntityManager.ToSharedRef(), GetClass(), UE::Mass::FSubsystemTypeTraits::Make<UInstancedActorsSubsystem>());
}

void UInstancedActorsSubsystem::Deinitialize()
{
#if WITH_EDITOR
	InstancedActorsCVars::CVarRefreshSettings.AsVariable()->SetOnChangedCallback(FConsoleVariableDelegate());
#endif

	Super::Deinitialize();

	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedActorsSubsystem Deinitialize);

	EntityManager.Reset();
	ExemplarActors.Reset();

	if (IsValid(ExemplarActorWorld))
	{
		ExemplarActorWorld->DestroyWorld(/*bInformEngineOfWorld*/false);
	}
}

void UInstancedActorsSubsystem::Tick(float DeltaTime)
{
	// Spawn entities for pending managers added in RequestDeferredSpawnEntities
	ExecutePendingDeferredSpawnEntitiesRequests(/*StopAfterSeconds*/InstancedActorsCVars::MaxDeferSpawnEntitiesTimePerTick);
}

TStatId UInstancedActorsSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UInstancedActorsSubsystem, STATGROUP_Tickables);
}

FInstancedActorsManagerHandle UInstancedActorsSubsystem::AddManager(AInstancedActorsManager& Manager)
{
	FInstancedActorsManagerHandle ManagerHandle;
	const FBox ManagerBounds = Manager.GetInstanceBounds();

	if (ensureMsgf(Algo::Find(Managers, &Manager) == nullptr, TEXT("A given Manager instance is not expected to be added twice")))
	{
		ManagerHandle = Managers.Add(&Manager);
		ManagersHashGrid.Add(ManagerHandle, ManagerBounds);

#if WITH_INSTANCEDACTORS_DEBUG
		// Record initial bounds so we can compare on removal to make sure it wasn't changed
		DebugManagerBounds.Add(&Manager, ManagerBounds);
#endif

		// Let Manager know the subsystem is ready. 
		//
		// Common callback for both AInstancedActorsManager::BeginPlay -> AddManager and latent 
		// UInstancedActorsSubsystem::Initialize -> AddManager
		Manager.OnAddedToSubsystem(*this, ManagerHandle);
	}
	else
	{
		ManagerHandle = Manager.GetManagerHandle();
		checkf(ManagerHandle.IsValid(), TEXT("If a given Manager has already been registered we expect it to host a valid ManagerHandle"));
	}
	
	return ManagerHandle;
}

void UInstancedActorsSubsystem::RemoveManager(const FInstancedActorsManagerHandle ManagerHandle)
{
	if (ensureMsgf(Managers.IsValidIndex(ManagerHandle.GetManagerID()), TEXT("Attempting to remove unknown manager (%d)"), ManagerHandle.GetManagerID()))
	{
		CancelDeferredSpawnEntitiesRequest(ManagerHandle);

		AInstancedActorsManager* Manager = Managers[ManagerHandle.GetManagerID()].Get();
		if (ensureMsgf(Manager != nullptr, TEXT("Attempting to remove invalid manager")))
		{

			const FBox ManagerBounds = Manager->GetInstanceBounds();
	
			Managers.RemoveAt(ManagerHandle.GetManagerID());
	
			ManagersHashGrid.Remove(ManagerHandle.GetManagerID(), ManagerBounds);

#if WITH_INSTANCEDACTORS_DEBUG
			// Compare to initial bounds to make sure it wasn't changed, as that would mean ManagersHashGrid.Remove above using latest
			// bounds, wouldn't have removed the manager from the grid.
			FBox OldManagerBounds;
			DebugManagerBounds.RemoveAndCopyValue(Manager, OldManagerBounds);
			ensureMsgf(ManagerBounds.Equals(OldManagerBounds), TEXT("Instanced Actor Manager (%s) has unexpectedly changed bounds (now: %s) since initial registration (was: %s). Movable managers are not supported"), *Manager->GetPathName(), *ManagerBounds.ToString(), *OldManagerBounds.ToString());
#endif
		}
	}
}

void UInstancedActorsSubsystem::RequestDeferredSpawnEntities(FInstancedActorsManagerHandle ManagerHandle)
{
	if (ensureMsgf(Managers.IsValidIndex(ManagerHandle.GetManagerID()), TEXT("Attempting to request deferred spawn entities for unknown manager (%d)"), ManagerHandle.GetManagerID()))
	{
		PendingManagersToSpawnEntities.Add(ManagerHandle);
	}
}

bool UInstancedActorsSubsystem::CancelDeferredSpawnEntitiesRequest(FInstancedActorsManagerHandle ManagerHandle)
{
	int32 NumRemoved = PendingManagersToSpawnEntities.Remove(ManagerHandle);
	return NumRemoved > 0;
}

bool UInstancedActorsSubsystem::ExecutePendingDeferredSpawnEntitiesRequests(double StopAfterSeconds)
{
	if (PendingManagersToSpawnEntities.IsEmpty())
	{
		return true;
	}

	const double TimeAllowedEnd = FMath::IsFinite(StopAfterSeconds) ? FPlatformTime::Seconds() + StopAfterSeconds : INFINITY;
	
	// Execute InitializeModifyAndSpawnEntities for pending managers
	int32 PendingRequestIndex = 0;
	for (; PendingRequestIndex < PendingManagersToSpawnEntities.Num(); ++PendingRequestIndex)
	{
		const FInstancedActorsManagerHandle& ManagerHandle = PendingManagersToSpawnEntities[PendingRequestIndex];
		if (ensureMsgf(Managers.IsValidIndex(ManagerHandle.GetManagerID()), TEXT("Attempting to perform deferred entity spawn for unknown manager (%d)"), ManagerHandle.GetManagerID()))
		{
			AInstancedActorsManager* Manager = Managers[ManagerHandle.GetManagerID()].Get();
			if (ensureMsgf(IsValid(Manager), TEXT("Attempting to perform deferred entity spawn for invalid manager (%d)"), ManagerHandle.GetManagerID()))
			{
				Manager->InitializeModifyAndSpawnEntities();
			}
		}

		// Stop after StopAfterSeconds
		if (FPlatformTime::Seconds() >= TimeAllowedEnd)
		{
			++PendingRequestIndex;
			break;
		}
	}

	// Remove processed requests
	const int32 NumProcessedRequests = PendingRequestIndex;
	PendingManagersToSpawnEntities.RemoveAt(0, NumProcessedRequests);

	const bool bExecutedAllPending = PendingManagersToSpawnEntities.IsEmpty();
	UE_CLOGF(!bExecutedAllPending, LogInstancedActors, Verbose, "UInstancedActorsSubsystem deferring %d remaining spawn entities requests to next frame", PendingManagersToSpawnEntities.Num());
	return bExecutedAllPending;
}

bool UInstancedActorsSubsystem::HasPendingDeferredSpawnEntitiesRequests() const
{
	return !PendingManagersToSpawnEntities.IsEmpty();
}

FInstancedActorsModifierVolumeHandle UInstancedActorsSubsystem::AddModifierVolume(UInstancedActorsModifierVolumeComponent& ModifierVolume)
{
	const FBox ModifierVolumeBounds = ModifierVolume.Bounds.GetBox();

	const int32 ModifierVolumeID = ModifierVolumes.Add(&ModifierVolume);

	FInstancedActorsModifierVolumeHandle ModifierVolumeHandle = ModifierVolumeID;
	ModifierVolumesHashGrid.Add(ModifierVolumeHandle, ModifierVolumeBounds);

#if WITH_INSTANCEDACTORS_DEBUG
	// Record initial bounds so we can compare on removal to make sure it wasn't changed
	DebugModifierVolumeBounds.Add(&ModifierVolume, ModifierVolumeBounds);
#endif

	// Let ModifierVolumeHandle know the subsystem is ready. 
	//
	// Common callback for both UInstancedActorsModifierVolumeComponent::BeginPlay -> AddModifierVolume and latent 	
	// UInstancedActorsSubsystem::Initialize -> AddModifierVolume
	ModifierVolume.OnAddedToSubsystem(*this, ModifierVolumeHandle);

	return ModifierVolumeHandle;
}

void UInstancedActorsSubsystem::RemoveModifierVolume(const FInstancedActorsModifierVolumeHandle ModifierVolumeHandle)
{
	if (ensureMsgf(ModifierVolumes.IsValidIndex(ModifierVolumeHandle.GetModifierVolumeID()), TEXT("Attempting to remove unknown modifier volume (%d)"), ModifierVolumeHandle.GetModifierVolumeID()))
	{
		UInstancedActorsModifierVolumeComponent* ModifierVolume = ModifierVolumes[ModifierVolumeHandle.GetModifierVolumeID()].Get();
		if (ensureMsgf(ModifierVolume != nullptr, TEXT("Attempting to remove invalid modifier volume")))
		{
			const FBox ModifierVolumeBounds = ModifierVolume->Bounds.GetBox();
	
			ModifierVolumes.RemoveAt(ModifierVolumeHandle.GetModifierVolumeID());
	
			ModifierVolumesHashGrid.Remove(ModifierVolumeHandle.GetModifierVolumeID(), ModifierVolumeBounds);

#if WITH_INSTANCEDACTORS_DEBUG
			// Compare to initial bounds to make sure it wasn't changed, as that would mean ModifierVolumesHashGrid.Remove 
			// above using latest bounds, wouldn't have removed the modifier volume from the grid.
			FBox OldModifierVolumeBounds;
			DebugModifierVolumeBounds.RemoveAndCopyValue(ModifierVolume, OldModifierVolumeBounds);
			ensureMsgf(ModifierVolumeBounds.Equals(OldModifierVolumeBounds), TEXT("Instanced Actor Modifier Volume (%s) has unexpectedly changed bounds (now: %s) since initial registration (was: %s). Movable modifier volumes are not supported"), *ModifierVolume->GetReadableName(), *ModifierVolumeBounds.ToString(), *OldModifierVolumeBounds.ToString());
#endif
		}
	}
}

#if WITH_EDITOR
FInstancedActorsInstanceHandle UInstancedActorsSubsystem::InstanceActor(TSubclassOf<AActor> ActorClass, FTransform InstanceTransform, ULevel* Level, const FGameplayTagContainer& AdditionalInstanceTags)
{
	return InstanceActor(ActorClass, InstanceTransform, Level, AdditionalInstanceTags, InstancedActorsManagerClass);
}

FInstancedActorsInstanceHandle UInstancedActorsSubsystem::InstanceActor(TSubclassOf<AActor> ActorClass, FTransform InstanceTransform, ULevel* Level, const FGameplayTagContainer& AdditionalInstanceTags, TSubclassOf<AInstancedActorsManager> ManagerClass)
{
	if (!ensureMsgf(Level, TEXT("Expecting a valid Level. Received nullptr.")))
	{
		return FInstancedActorsInstanceHandle();
	}

	UWorld* World = Level->GetWorld();
	if (!ensureMsgf(!World->IsGameWorld(), TEXT("Instanced Actors doesn't yet support runtime addition of instances. Skipping instance creation")))
	{
		return FInstancedActorsInstanceHandle();
	}
	else if (!ensureMsgf(ActorClass, TEXT("Expecting a valid ActorClass. Received None.")))
	{
		return FInstancedActorsInstanceHandle();
	}

	if (!ManagerClass)
	{
		if (!ensureMsgf(InstancedActorsManagerClass, TEXT("%hs called with ManagerClass being None and default InstancedActorsManagerClass not being set"), __FUNCTION__))
		{
			return FInstancedActorsInstanceHandle();
		}
		ManagerClass = InstancedActorsManagerClass;
	}

	// Ensure settings presence for ActorClass
	if (InstancedActorsCVars::EditorEnforceActorClassSettingsPresence > 0)
	{
		const bool bFoundClassSettings = DoesActorClassHaveRegisteredSettings(ActorClass);
		if (!bFoundClassSettings)
		{
			FMessageLog MessageLog("InstancedActors");
			switch (InstancedActorsCVars::EditorEnforceActorClassSettingsPresence)
			{
				// 1 = Log a message log warning, continue to instance ActorClass regardless
				case 1:
				{
					MessageLog.AddMessage(
						FTokenizedMessage::Create(EMessageSeverity::Warning)
							->AddToken(FUObjectToken::Create(ActorClass))
							->AddToken(FTextToken::Create(NSLOCTEXT("InstancedActors", "CantInstanceClassWarning", "doesn't have a matching class or super class entry in the ActorClassSettingsRegistry.")))
					);

					MessageLog.Open(EMessageSeverity::Warning);

					break;
				}
				// 2 = Log a message log error, skip instancing ActorClass
				// 3 = Ensure (log stack trace and break debugger), log a message log error, skip instancing ActorClass
				case 2:
				case 3:
				{
					MessageLog.AddMessage(
						FTokenizedMessage::Create(EMessageSeverity::Error)
						->AddToken(FUObjectToken::Create(ActorClass))
						->AddToken(FTextToken::Create(NSLOCTEXT("InstancedActors", "CantInstanceClassError", "doesn't have a matching class or super class entry in the ActorClassSettingsRegistry, skipping instance of 'unknown' type.")))
					);

					if (InstancedActorsCVars::EditorEnforceActorClassSettingsPresence >= 3)
					{
						ensureMsgf(bFoundClassSettings, TEXT("No instanced InstancedActorsClassSettings entry found in ActorClassSettingsRegistry for %s or it's super classes, skipping instance of 'unknown' type."), *ActorClass->GetPathName());
					}

					MessageLog.Open(EMessageSeverity::Warning);

					return FInstancedActorsInstanceHandle();
				}
			}
		}
	}

	// Compute the manager grid cell coords for this instance
	//
	// NOTE: Traditional (non partitioned) worlds will only get ONE IAM at the origin no 
	// matter the size of the grid specified or where the instance to be added is located.
	const uint32 ManagerGridSize = ManagerClass.GetDefaultObject()->GetDefaultGridSize(Level->GetWorld());
	const UActorPartitionSubsystem::FCellCoord CellCoord = UActorPartitionSubsystem::FCellCoord::GetCellCoord(InstanceTransform.GetTranslation(), Level, ManagerGridSize);
	FVector CellCenter(ForceInitToZero);
	
	// If this is a world partition world we want to be in the centre of a cell.
	const bool bIsPartitionedLevel = (Level->GetWorldPartitionRuntimeCell() != nullptr);
	const bool bIsPartitionedWorld = bIsPartitionedLevel || Level->GetWorld()->IsPartitionedWorld();
	if (bIsPartitionedWorld)
	{
		FBox CellBounds = UActorPartitionSubsystem::FCellCoord::GetCellBounds(CellCoord, ManagerGridSize);
		CellCenter = CellBounds.GetCenter();
	}

	// Note: These will be re-compiled at runtime in UInstancedActorsData::BeginPlay, and may differ as such.
	FSharedStruct SharedSettings = GetOrCompileSettingsForActorClass(ActorClass);
	const FInstancedActorsSettings& Settings = SharedSettings.Get<FInstancedActorsSettings>();

	// Use the grid dictated by the settings, otherwise leave it as None.
	// None always resolves to the world's default (first) runtime grid in both the spatial hash and hash set
	// This matches the placement of a spawner with no explicit RuntimeGrid (e.g. a PCG actor on grid None).
	FName ManagerGrid = Settings.bOverride_OverrideWorldPartitionGrid ? Settings.OverrideWorldPartitionGrid : NAME_None;

	// We generate a guid (don't ask) so that WP can differentiate between PartitionActors in different world partition grids
	// but we only need to use that when this is a WP world.
	FArchiveMD5 ArMD5;
	ArMD5 << ManagerGrid;
	const FGuid ManagerGuid = bIsPartitionedWorld ? ArMD5.GetGuidFromHash() : FGuid();

	if (ensure(ActorPartitionSubsystem))
	{
		// Get or create manager for the instance's cell
		APartitionActor* PartitionActor = ActorPartitionSubsystem->GetActor(ManagerClass, CellCoord, /*bInCreate*/true, /*InGuid*/ManagerGuid, /*InGridSize*/ManagerGridSize, /*bInBoundsSearch*/true, 
		    /*InActorCreated*/[&bIsPartitionedWorld, &CellCenter, &ManagerGrid, &ManagerGuid](APartitionActor* NewPartitionActor)
		    {
		        AInstancedActorsManager* NewManager = CastChecked<AInstancedActorsManager>(NewPartitionActor);
				if (bIsPartitionedWorld)
				{
					NewManager->SetRuntimeGrid(ManagerGrid);
					NewManager->SetGridGuid(ManagerGuid);
				}
				NewManager->SetActorLocation(CellCenter);
			});
		if (ensureMsgf(IsValid(PartitionActor), TEXT("Failed spawning AInstancedActorsManager using UActorPartitionSubsystem::GetActor(bInCreate=true) to add instance to")))
		{
			AInstancedActorsManager* Manager = CastChecked<AInstancedActorsManager>(PartitionActor);
	
			// Add instance to manager
			return Manager->AddActorInstance(ActorClass, InstanceTransform, /*bWorldSpace*/true, AdditionalInstanceTags);
		}
	}

	return FInstancedActorsInstanceHandle();
}

bool UInstancedActorsSubsystem::RemoveActorInstance(const FInstancedActorsInstanceHandle& InstanceHandle, bool bDestroyManagerIfEmpty)
{
	AInstancedActorsManager* Manager = InstanceHandle.GetManager();
	if (ensure(IsValid(Manager)))
	{
		bool bRemoved = Manager->RemoveActorInstance(InstanceHandle);
		if (bRemoved)
		{
			if (bDestroyManagerIfEmpty && !Manager->HasAnyValidInstances())
			{
				Manager->Destroy();
			}

			return true;
		}
	}

	return false;
}
#endif // WITH_EDITOR

void UInstancedActorsSubsystem::ForEachManager(const FBox& QueryBounds, TFunctionRef<bool(AInstancedActorsManager&)>InOperation, TSubclassOf<AInstancedActorsManager> ManagerClass) const
{
	// Find roughly overlapping managers in the hash grid
	TArray<FInstancedActorsManagerHandle> OverlappedManagerHandles;
	ManagersHashGrid.Query(QueryBounds, OverlappedManagerHandles);

	for (const FInstancedActorsManagerHandle ManagerHandle : OverlappedManagerHandles)
	{
		const TWeakObjectPtr<AInstancedActorsManager>& Manager = Managers[ManagerHandle.GetManagerID()];
		if (Manager.IsValid())
		{
			// Exacting bounds intersection check
			if (Manager->GetInstanceBounds().Intersect(QueryBounds))
			{
				// Run InOperation for Manager
				const bool bContinue = InOperation(*Manager);
				if (!bContinue)
				{
					break;
				}
			}
		}
	}
}

void UInstancedActorsSubsystem::ForEachModifierVolume(const FBox& QueryBounds, TFunctionRef<bool(UInstancedActorsModifierVolumeComponent&)>InOperation) const
{
	// Find roughly overlapping modifier volumes in the hash grid
	TArray<FInstancedActorsModifierVolumeHandle> OverlappedModifierVolumeHandles;
	ModifierVolumesHashGrid.Query(QueryBounds, OverlappedModifierVolumeHandles);

	for (const FInstancedActorsModifierVolumeHandle ModifierVolumeHandle : OverlappedModifierVolumeHandles)
	{
		const TWeakObjectPtr<UInstancedActorsModifierVolumeComponent>& ModifierVolume = ModifierVolumes[ModifierVolumeHandle.GetModifierVolumeID()];
		if (ModifierVolume.IsValid())
		{
			// Exacting bounds intersection check
			if (ModifierVolume->Bounds.GetBox().Intersect(QueryBounds))
			{
				// Run InOperation for ModifierVolume
				const bool bContinue = InOperation(*ModifierVolume);
				if (!bContinue)
				{
					break;
				}
			}
		}
	}
}

void UInstancedActorsSubsystem::ForEachInstance(const FBox& QueryBounds, TFunctionRef<bool(const FInstancedActorsInstanceHandle&, const FTransform&, FInstancedActorsIterationContext&)> InOperation) const
{
	ForEachManager(QueryBounds, [&QueryBounds, &InOperation](AInstancedActorsManager& Manager)
	{
		bool bContinue = Manager.ForEachInstance(QueryBounds, InOperation);
		return bContinue;
	});
}

bool UInstancedActorsSubsystem::HasInstancesOfClass(const FBox& QueryBounds, TSubclassOf<AActor> ActorClass
	, const bool bTestActorsIfSpawned, const EInstancedActorsBulkLODMask AllowedLODs) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedActorsSubsystem::HasInstancesOfClass);
	
	UE_VLOG_BOX(this, LogInstancedActors, Log, QueryBounds, FColor::Orange, TEXT(""));

	bool bHasInstances = false;
	ForEachManager(QueryBounds, [QueryBounds, ActorClass, &bHasInstances, bTestActorsIfSpawned, AllowedLODs](AInstancedActorsManager& Manager)
	{
		bHasInstances = Manager.HasInstancesOfClass(QueryBounds, ActorClass, bTestActorsIfSpawned, AllowedLODs);
		const bool bContinue = !bHasInstances;
		return bContinue;
	});

	return bHasInstances;
}

TSharedRef<UE::InstancedActors::FExemplarActorData> UInstancedActorsSubsystem::GetOrCreateExemplarActor(TSubclassOf<AActor> ActorClass)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedActorsSubsystem GetOrCreateExemplarActor);

	UClass* ActorClassPtr = ActorClass.Get();
	check(ActorClassPtr);

	// Return existing?
	const TObjectKey<const UClass> ActorClassKey(ActorClassPtr);
	const uint32 ActorClassHash = GetTypeHash(ActorClassKey);
	
	if (const TWeakPtr<UE::InstancedActors::FExemplarActorData>* CachedExemplarActorDataPtr = ExemplarActors.FindByHash(ActorClassHash, ActorClassKey))
	{
		// This can fail in editor with undo/redo in the mix.
		TSharedPtr<UE::InstancedActors::FExemplarActorData> CachedExemplarActorData = CachedExemplarActorDataPtr->Pin();
		if (CachedExemplarActorData.IsValid() && CachedExemplarActorData->Actor.Get() != nullptr)
		{
			return CachedExemplarActorData.ToSharedRef();
		}
		else
		{
			// The examplar is not valid, we'll remove it and then re-create it below.
			ExemplarActors.RemoveByHash(ActorClassHash, ActorClassKey);
		}
	}

	// Lazy create a new 'inactive' UWorld to spawn fully constructed 'exemplar' actors in for
	// exemplary instance data introspection
	//
	// mz@todo IA: reconsider
	// @todo Move this back to Initialize and only create this subsystem for game worlds.
	//		 We're lazy creating this here after reports it's causing issues in other game worlds.
	//		 We currently can't limit the creation of this subsystem based on GFP loading state
	//		 as GFP plugin activation occurs later than AInstancedActorsManager::PostInitializeComponents
	//		 where we need to able to access this subsytem already.
	if (!IsValid(ExemplarActorWorld))
	{
		checkNoRecursion();

		UWorld::InitializationValues IVS;
		IVS.InitializeScenes(false);
		IVS.AllowAudioPlayback(false);
		IVS.RequiresHitProxies(false);
		IVS.CreatePhysicsScene(false);
		IVS.CreateNavigation(false);
		IVS.CreateAISystem(false);
		IVS.ShouldSimulatePhysics(false);
		IVS.EnableTraceCollision(false);
		IVS.SetTransactional(false);
		IVS.CreateFXSystem(false);

		UWorld& World = GetWorldRef();
		ExemplarActorWorld = UWorld::CreateWorld(EWorldType::Inactive,
			/*bInformEngineOfWorld*/false,
			/*WorldName*/TEXT("InstancedActorsSubsystem_ExemplarActorWorld"),
			/*Package*/nullptr,
			/*bAddToRoot*/false,
			World.GetFeatureLevel(),
			&IVS);
	}

	// Spawn new exemplar actor
	FActorSpawnParameters SpawnParameters;
	SpawnParameters.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;

	check(ExemplarActorWorld);
	AActor* NewExemplarActor = ExemplarActorWorld->SpawnActor(ActorClassPtr, /*Transform*/nullptr, SpawnParameters);
	check(NewExemplarActor);

	// Cache for subsequent calls
	TSharedPtr<UE::InstancedActors::FExemplarActorData> NewExemplarActorDataPtr{new UE::InstancedActors::FExemplarActorData{NewExemplarActor, this}};
	ExemplarActors.AddByHash(ActorClassHash, ActorClassKey, NewExemplarActorDataPtr);

	return NewExemplarActorDataPtr.ToSharedRef();
}

void UInstancedActorsSubsystem::UnregisterExemplarActorClass(TSubclassOf<AActor> ActorClass)
{
	const UClass* const ActorClassPtr = ActorClass.Get();
	check(ActorClassPtr);
	const TObjectKey<const UClass> ActorClassKey(ActorClassPtr);
	
	const uint32 ActorClassHash = GetTypeHash(ActorClassKey);	
	if (const TWeakPtr<UE::InstancedActors::FExemplarActorData>* CachedExemplarActorDataPtr = ExemplarActors.FindByHash(ActorClassHash, ActorClassPtr))
	{
		ExemplarActors.RemoveByHash(ActorClassHash, ActorClassKey);
	}
}

FSharedStruct UInstancedActorsSubsystem::GetOrCompileSettingsForActorClass(TSubclassOf<AActor> ActorClass)
{
	// Return cached?
	if (const FSharedStruct* CachedActorClassSettings = PerActorClassSettings.Find(ActorClass.Get()))
	{
		return *CachedActorClassSettings;
	}

	// Compile and cache new settings
	FSharedStruct ActorClassSettings = CompileSettingsForActorClass(ActorClass);

	const FInstancedActorsSettings& Settings = ActorClassSettings.Get<FInstancedActorsSettings>();

	PerActorClassSettings.Add(ActorClass.Get(), ActorClassSettings);
	return ActorClassSettings;
}

bool UInstancedActorsSubsystem::DoesActorClassHaveRegisteredSettings(TSubclassOf<AActor> ActorClass, bool bIncludeSuperClasses)
{
	check(DataRegistrySubsystem);
	check(ProjectSettings);

	// Apply class-specific settings, walking up the inheritance hierarchy starting with ActorClass
	UClass* ClassOrSuperClass = ActorClass.Get();
	while (ClassOrSuperClass != nullptr)
	{
		// Find FInstancedActorsClassSettingsBase for ClassOrSuperClass
		// Note: For fast lookup, we use the classes FName to lookup class settings, requiring class names to be unique for per-class settings
		const FInstancedActorsClassSettingsBase* ClassOrSuperClassSettings = DataRegistrySubsystem->GetCachedItem<FInstancedActorsClassSettingsBase>({ProjectSettings->ActorClassSettingsRegistryType, ClassOrSuperClass->GetFName()});
		if (ClassOrSuperClassSettings != nullptr)
		{
			return true;
		}

		ClassOrSuperClass = bIncludeSuperClasses ? ClassOrSuperClass->GetSuperClass() : nullptr;
	}

	return false;
}

FSharedStruct UInstancedActorsSubsystem::CompileSettingsForActorClass(TSubclassOf<AActor> ActorClass) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedActorsSubsystem CompileSettingsForActorClass);

	check(DataRegistrySubsystem);
	check(ProjectSettings);

	// Start with default constructed settings as basis for all settings
	FSharedStruct CompiledSettings;
	CompiledSettings.InitializeAs(SettingsType.Get());

	// Apply override settings in reverse order / highest priority first, to allow us to walk up the 
	// class hierarchy applying progressively lower priority class settings

	auto GetCachedItem = [DataRegistrySubsystem = DataRegistrySubsystem](const FDataRegistryId& ItemId)
		{
			const uint8* TempItemMemory = nullptr;
			const UScriptStruct* TempItemStuct = nullptr;

			if (DataRegistrySubsystem->GetCachedItemRaw(TempItemMemory, TempItemStuct, ItemId))
			{
				if (!ensureMsgf(TempItemStuct->IsChildOf(FInstancedActorsSettings::StaticStruct())
					, TEXT("Can't cast data item of type %s to %s! Code should check type before calling GetCachedDataRegistryItem")
					, *TempItemStuct->GetName(), *FInstancedActorsSettings::StaticStruct()->GetName()))
				{
					return FInstancedStruct::Make<FInstancedActorsSettings>();
				}

				FInstancedStruct InstantStruct;
				InstantStruct.InitializeAs(TempItemStuct, TempItemMemory);
				return InstantStruct;
			}

			return FInstancedStruct::Make<FInstancedActorsSettings>();
		};

	// First start with highest priority EnforcedSettings overrides, if specified
	if (!ProjectSettings->EnforcedSettingsName.IsNone())
	{
		const FInstancedStruct EnforcedSettings = GetCachedItem({ ProjectSettings->NamedSettingsRegistryType, ProjectSettings->EnforcedSettingsName });
		CompiledSettings.Get<FInstancedActorsSettings>().OverrideIfDefault(EnforcedSettings, ProjectSettings->EnforcedSettingsName);
	}

	// Apply class-specific settings, walking up the inheritance hierarchy starting with ActorClass
	bool bFoundClassSettings = false;
	UClass* ClassOrSuperClass = ActorClass.Get();
	while (ClassOrSuperClass != nullptr)
	{
		// Find FInstancedActorsClassSettingsBase for ClassOrSuperClass
		// Note: For fast lookup, we use the classes FName to lookup class settings, requiring class names to be unique for per-class settings
		const FInstancedActorsClassSettingsBase* ClassOrSuperClassSettings = DataRegistrySubsystem->GetCachedItem<FInstancedActorsClassSettingsBase>({ProjectSettings->ActorClassSettingsRegistryType, ClassOrSuperClass->GetFName()});
		if (ClassOrSuperClassSettings != nullptr)
		{
			bFoundClassSettings = true;

			// Apply class OverrideSettings
			CompiledSettings.Get<FInstancedActorsSettings>().OverrideIfDefault(ClassOrSuperClassSettings->MakeOverrideSettings(), ClassOrSuperClass->GetFName());

			// Apple class BaseSettings in reverse order
			for (const FName& BaseSettingsName : ReverseIterate(ClassOrSuperClassSettings->BaseSettings))
			{
				const FInstancedStruct BaseSettings = GetCachedItem({ ProjectSettings->NamedSettingsRegistryType, BaseSettingsName });

				if (ensureMsgf(BaseSettings.IsValid(), TEXT("FInstancedActorsClassSettingsBase (%s) references unknown named settings '%s', skipping.")
					, *ClassOrSuperClass->GetPathName(), *BaseSettingsName.ToString()))
				{
					CompiledSettings.Get<FInstancedActorsSettings>().OverrideIfDefault(BaseSettings, BaseSettingsName);
				}
			}
		}

		ClassOrSuperClass = ClassOrSuperClass->GetSuperClass();
	}

	// No class settings found?
	if (!bFoundClassSettings && InstancedActorsCVars::RuntimeEnforceActorClassSettingsPresence > 0 && GetWorldRef().IsGameWorld())
	{
		if (InstancedActorsCVars::RuntimeEnforceActorClassSettingsPresence >= 2)
		{
			ensureMsgf(bFoundClassSettings, TEXT("No instanced InstancedActorsClassSettings entry found in ActorClassSettingsRegistry for %s or it's super classes"), *ActorClass->GetPathName());
		}
		else
		{
			UE_LOGF(LogInstancedActors, Error, "No instanced InstancedActorsClassSettings entry found in ActorClassSettingsRegistry for %ls or it's super classes", *ActorClass->GetPathName());
		}
	}

	// Lastly, apply lowest priority project DefaultBaseSettings if specified
	if (!ProjectSettings->DefaultBaseSettingsName.IsNone())
	{
		const FInstancedStruct DefaultBaseSettings = GetCachedItem({ ProjectSettings->NamedSettingsRegistryType, ProjectSettings->DefaultBaseSettingsName });

		if (ensureMsgf(DefaultBaseSettings.IsValid(), TEXT("UInstancedActorsProjectSettings DefaultBaseSettingsName references unknown named settings '%s', skipping.")
			, *ProjectSettings->DefaultBaseSettingsName.ToString()))
		{
			CompiledSettings.Get<FInstancedActorsSettings>().OverrideIfDefault(DefaultBaseSettings, ProjectSettings->DefaultBaseSettingsName);
		}
	}

	return CompiledSettings;
}

#if WITH_EDITOR
void UInstancedActorsSubsystem::HandleRefreshSettings(IConsoleVariable* InCVar)
{
	// Emptying the Map because FInstancedActorsSettings::OverrideIfDefault checks its bOverride_ members before updating them. This means we can only set them once, and never again.
	PerActorClassSettings.Empty();

	for (TObjectIterator<UInstancedActorsData> It; It; ++It)
	{
		UInstancedActorsData* InstancedActorsData = *It;
		if (IsValid(InstancedActorsData) && !InstancedActorsData->IsTemplate())
		{
			InstancedActorsData->SetSharedSettings(GetOrCompileSettingsForActorClass(InstancedActorsData->ActorClass));
		} 
	} 
}
#endif

void UInstancedActorsSubsystem::MarkInstanceRepresentationDirty(FInstancedActorsInstanceHandle InstanceHandle)
{
	if (!ensure(InstanceHandle.IsValid()))
	{
		return;
	}

	DirtyRepresentationInstances.Add(InstanceHandle);
}

void UInstancedActorsSubsystem::PopAllDirtyRepresentationInstances(TArray<FInstancedActorsInstanceHandle>& OutInstances)
{
	OutInstances.Append(DirtyRepresentationInstances);
	DirtyRepresentationInstances.Reset();
}

FInstancedActorsVisualizationDesc UInstancedActorsSubsystem::CreateVisualDescriptionFromActor(const AActor& ExemplarActor) const
{
	return FInstancedActorsVisualizationDesc::FromActor(ExemplarActor);
}

TArray<UInstancedActorsSubsystem::FNextTickSharedFragment>& UInstancedActorsSubsystem::GetTickableSharedFragments()
{	
	RegisterNewSharedFragmentsInternal();
	return SortedSharedFragments;
}

void UInstancedActorsSubsystem::UpdateAndResetTickTime(TConstStructView<FInstancedActorsDataSharedFragment> InstancedActorsDataSharedFragment)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInstancedActorsSubsystem_UpdateTickableSharedFragments);

	const bool bParamStructFound = RegisterNewSharedFragmentsInternal(InstancedActorsDataSharedFragment);
	if (bParamStructFound == false)
	{
		// we need to find the one. Naive implementation for now.
		// starting from back in assumption stuff has been removed and re-added so the relevant data should be closer 
		// to the back due to being at EMassLOD::Off level
		for (int32 Index = SortedSharedFragments.Num() - 1; Index >= 0; --Index)
		{
			if (SortedSharedFragments[Index].SharedStruct == InstancedActorsDataSharedFragment)
			{
				FNextTickSharedFragment TickFragment = SortedSharedFragments[Index];
				// setting to 0 will force update the very next time Batch LOD is being calculated. 
				TickFragment.NextTickTime = 0;
				SortedSharedFragments.HeapRemoveAt(Index, EAllowShrinking::No);
				SortedSharedFragments.HeapPush(MoveTemp(TickFragment));
				break;
			}
		}
	}
}

void UInstancedActorsSubsystem::MarkIADDirtyWithEntitiesPendingDestruction(UInstancedActorsData& InstancedActorsData)
{
	FScopeLock Lock(&DirtyIadWithEntitiesPendingDestructionCS);
	DirtyIadWithEntitiesPendingDestruction.AddUnique(&InstancedActorsData);
}

void UInstancedActorsSubsystem::PopAllDirtyIADWithEntitiesPendingDestruction(TArray<TWeakObjectPtr<UInstancedActorsData>>& OutInstancedActorsData)
{
	FScopeLock Lock(&DirtyIadWithEntitiesPendingDestructionCS);
	OutInstancedActorsData.Append(MoveTemp(DirtyIadWithEntitiesPendingDestruction));
}

bool UInstancedActorsSubsystem::RegisterNewSharedFragmentsInternal(TConstStructView<FInstancedActorsDataSharedFragment> InstancedActorsDataSharedFragment)
{
	check(EntityManager);

	const bool bParamStructProvided = InstancedActorsDataSharedFragment.IsValid();
	// starting with !bParamStructProvided will result in short-circuiting the assignment operation below if InstancedActorsDataSharedFragment is empty
	bool bParamStructFound = !bParamStructProvided;
	TConstArrayView<FSharedStruct> AllSharedFragmentsOfType = EntityManager->GetSharedFragmentsOfType<FInstancedActorsDataSharedFragment>();
	
	if (SortedSharedFragments.Num() < AllSharedFragmentsOfType.Num())
	{
		// We add all of them at the front for immediate processing.
		const int32 StartingIndex = SortedSharedFragments.Num();
		const int32 NewItemsCount = (AllSharedFragmentsOfType.Num() - SortedSharedFragments.Num());
		SortedSharedFragments.InsertDefaulted(0, NewItemsCount);
		for (int32 NewIndex = 0; NewIndex < NewItemsCount; ++NewIndex)
		{
			SortedSharedFragments[NewIndex].SharedStruct = AllSharedFragmentsOfType[StartingIndex + NewIndex];
			bParamStructFound = bParamStructFound || (AllSharedFragmentsOfType[StartingIndex + NewIndex] == InstancedActorsDataSharedFragment);
		}
		SortedSharedFragments.Heapify();
	}
	check(SortedSharedFragments.Num() == AllSharedFragmentsOfType.Num());

	return bParamStructFound;
}
