// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsTypes.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"
#include "Engine/World.h"
#include "InstancedActorsSettings.h"
#include "ServerInstancedActorsSpawnerSubsystem.h"
#include "ClientInstancedActorsSpawnerSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsTypes)

DEFINE_LOG_CATEGORY(LogInstancedActors);

namespace UE::InstancedActors::Utils
{
	TSubclassOf<UMassActorSpawnerSubsystem> DetermineActorSpawnerSubsystemClass(const UWorld& World)
	{
		// @todo Add support for non-replay NM_Standalone where we should use UServerInstancedActorsSpawnerSubsystem for 
		// authoritative actor spawning.
		if (World.GetNetMode() == NM_Client)
		{
			return GET_INSTANCEDACTORS_CONFIG_VALUE(GetClientActorSpawnerSubsystemClass());
		}
		return GET_INSTANCEDACTORS_CONFIG_VALUE(GetServerActorSpawnerSubsystemClass());
	}

	UServerInstancedActorsSpawnerSubsystem* GetServerInstancedActorsSpawnerSubsystem(const UWorld& World)
	{
		TSubclassOf<UMassActorSpawnerSubsystem> SpawnerSubsystemClass = GET_INSTANCEDACTORS_CONFIG_VALUE(GetServerActorSpawnerSubsystemClass());
		check(SpawnerSubsystemClass);
		return Cast<UServerInstancedActorsSpawnerSubsystem>(World.GetSubsystemBase(SpawnerSubsystemClass));
	}

	UClientInstancedActorsSpawnerSubsystem* GetClientInstancedActorsSpawnerSubsystem(const UWorld& World)
	{
		TSubclassOf<UMassActorSpawnerSubsystem> SpawnerSubsystemClass = GET_INSTANCEDACTORS_CONFIG_VALUE(GetClientActorSpawnerSubsystemClass());
		check(SpawnerSubsystemClass);
		return Cast<UClientInstancedActorsSpawnerSubsystem>(World.GetSubsystemBase(SpawnerSubsystemClass));
	}

	UMassActorSpawnerSubsystem* GetActorSpawnerSubsystem(const UWorld& World)
	{
		if (World.GetNetMode() == NM_Client)
		{
			return GetClientInstancedActorsSpawnerSubsystem(World);
		}
		return GetServerInstancedActorsSpawnerSubsystem(World);
	}

	UInstancedActorsSubsystem* GetInstancedActorsSubsystem(const UWorld& World)
	{
		TSubclassOf<UInstancedActorsSubsystem> InstancedActorsSubsystemClass = GET_INSTANCEDACTORS_CONFIG_VALUE(GetInstancedActorsSubsystemClass());
		check(InstancedActorsSubsystemClass);

		UInstancedActorsSubsystem* InstancedActorSubsystem = Cast<UInstancedActorsSubsystem>(World.GetSubsystemBase(InstancedActorsSubsystemClass));

		return InstancedActorSubsystem;
	}
} // namespace UE::InstancedActors::Utils

//-----------------------------------------------------------------------------
// FInstancedActorsTagSet
//-----------------------------------------------------------------------------
FInstancedActorsTagSet::FInstancedActorsTagSet(const FGameplayTagContainer& InTags)
{
	TArray<FGameplayTag> SortedTags;
	InTags.GetGameplayTagArray(SortedTags);
	SortedTags.Sort();

	Tags = FGameplayTagContainer::CreateFromArray(SortedTags);

	CalcHash();
}

void FInstancedActorsTagSet::CalcHash()
{
	uint32 NewHash = 0;
	for (const FGameplayTag& Tag : Tags.GetGameplayTagArray())
	{
		NewHash = HashCombine(NewHash, GetTypeHash(Tag));
	}
	
	Hash = NewHash;
}

//-----------------------------------------------------------------------------
// FStaticMeshInstanceVisualizationDesc
//-----------------------------------------------------------------------------
FInstancedActorsVisualizationDesc::FInstancedActorsVisualizationDesc(const FInstancedActorsSoftVisualizationDesc& SoftVisualizationDesc)
{
	for (const FSoftISMComponentDescriptor& SoftISMComponentDescriptor : SoftVisualizationDesc.ISMComponentDescriptors)
	{
		// Calls FISMComponentDescriptor(FSoftISMComponentDescriptor&) which will LoadSynchronous any soft paths
		ISMComponentDescriptors.Emplace(SoftISMComponentDescriptor);
	}
}

FInstancedActorsVisualizationDesc FInstancedActorsVisualizationDesc::FromActor(const AActor& ExemplarActor
	, const FAdditionalSetupStepsFunction& AdditionalSetupSteps)
{
	return FromActor(ExemplarActor, [&AdditionalSetupSteps](const AActor& ExemplarActor, FInstancedActorsVisualizationDesc& OutVisualization)
	{
		if (OutVisualization.ISMComponentDescriptors.Num() > 0)
		{
			AdditionalSetupSteps(ExemplarActor, OutVisualization.ISMComponentDescriptors[0], OutVisualization);
		}
	});
}

FInstancedActorsVisualizationDesc FInstancedActorsVisualizationDesc::FromActor(const AActor& ExemplarActor,
	const FVisualizationDescSetupFunction& AdditionalSetupSteps)
{
	FInstancedActorsVisualizationDesc Visualization;
	USceneComponent* RootComponent = ExemplarActor.GetRootComponent();
	
	ExemplarActor.ForEachComponent<UStaticMeshComponent>(/*bIncludeFromChildActors=*/false, [&Visualization, RootComponent](UStaticMeshComponent* SourceStaticMeshComponent)
	{
		if (!SourceStaticMeshComponent->IsVisible())
		{
			return;
		}

		const UStaticMesh* StaticMesh = SourceStaticMeshComponent->GetStaticMesh();
		if (!IsValid(StaticMesh))
		{
			// No mesh = no visualization
			return;
		}

		FISMComponentDescriptor& ISMComponentDescriptor = Visualization.ISMComponentDescriptors.AddDefaulted_GetRef();
		ISMComponentDescriptor.InitFrom(SourceStaticMeshComponent);

		// LocalTransform means local to the Actor/Entity, so we need to compute based on the UStaticMeshComponent's relative transform accordingly
		// (in case this UStaticMeshComponent was a child of another UStaticMeshComponent within the Actor hierarchy)
		if (SourceStaticMeshComponent != RootComponent)
		{
			ISMComponentDescriptor.LocalTransform = SourceStaticMeshComponent->GetComponentToWorld();
		}
	});

	AdditionalSetupSteps(ExemplarActor, Visualization);

	return MoveTemp(Visualization);
}

FStaticMeshInstanceVisualizationDesc FInstancedActorsVisualizationDesc::ToMassVisualizationDesc() const
{
	FStaticMeshInstanceVisualizationDesc OutMassVisualizationDesc;

	for (const FISMComponentDescriptor& ISMComponentDescriptor : ISMComponentDescriptors)
	{
		if (!ensure(IsValid(ISMComponentDescriptor.StaticMesh)))
		{
			continue;
		}

		FMassStaticMeshInstanceVisualizationMeshDesc& MeshDesc = OutMassVisualizationDesc.Meshes.AddDefaulted_GetRef();
		MeshDesc.Mesh = ISMComponentDescriptor.StaticMesh;
		MeshDesc.LocalTransform = ISMComponentDescriptor.LocalTransform;
		MeshDesc.bCastShadows = ISMComponentDescriptor.bCastShadow;
		MeshDesc.Mobility = EComponentMobility::Stationary;
		MeshDesc.MaterialOverrides = ISMComponentDescriptor.OverrideMaterials;
		MeshDesc.ISMComponentClass = UInstancedStaticMeshComponent::StaticClass();
	}

	OutMassVisualizationDesc.CustomDataFloats = CustomDataFloats;

	return MoveTemp(OutMassVisualizationDesc);
}

//-----------------------------------------------------------------------------
// FInstancedActorsSoftVisualizationDesc
//-----------------------------------------------------------------------------
FInstancedActorsSoftVisualizationDesc::FInstancedActorsSoftVisualizationDesc(const FInstancedActorsVisualizationDesc& VisualizationDesc)
{
	for (const FISMComponentDescriptor& ISMComponentDescriptor : VisualizationDesc.ISMComponentDescriptors)
	{
		ISMComponentDescriptors.Emplace(ISMComponentDescriptor);
	}
}

void FInstancedActorsSoftVisualizationDesc::GetAssetsToLoad(TArray<FSoftObjectPath>& OutAssetsToLoad) const
{
	for (const FSoftISMComponentDescriptor& ISMComponentDescriptor : ISMComponentDescriptors)
	{
		if (ISMComponentDescriptor.StaticMesh.IsPending())
		{
			OutAssetsToLoad.Add(ISMComponentDescriptor.StaticMesh.ToSoftObjectPath());
		}
		for (const TSoftObjectPtr<UMaterialInterface>& OverrideMaterial : ISMComponentDescriptor.OverrideMaterials)
		{
			if (OverrideMaterial.IsPending())
			{
				OutAssetsToLoad.Add(OverrideMaterial.ToSoftObjectPath());
			}
		}
		if (ISMComponentDescriptor.OverlayMaterial.IsPending())
		{
			OutAssetsToLoad.Add(ISMComponentDescriptor.OverlayMaterial.ToSoftObjectPath());
		}
		for (const TSoftObjectPtr<URuntimeVirtualTexture>& RuntimeVirtualTexture : ISMComponentDescriptor.RuntimeVirtualTextures)
		{
			if (RuntimeVirtualTexture.IsPending())
			{
				OutAssetsToLoad.Add(RuntimeVirtualTexture.ToSoftObjectPath());
			}
		}
	}
}
