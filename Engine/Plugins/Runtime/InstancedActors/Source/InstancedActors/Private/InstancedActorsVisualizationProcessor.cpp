// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsVisualizationProcessor.h"
#include "MassStationaryISMSwitcherProcessor.h"
#include "InstancedActorsTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsVisualizationProcessor)


UInstancedActorsVisualizationProcessor::UInstancedActorsVisualizationProcessor()
{
	bAutoRegisterWithProcessingPhases = true;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllNetModes);
	// This processor needs to be executed before UMassStationaryISMSwitcherProcessors since that's the processor
	// responsible for executing what UInstancedActorsVisualizationProcessor calculates.
	// Missing this dependency would result in client-side one-frame representation absence when switching
	// from actor representation back to ISM.
	ExecutionOrder.ExecuteBefore.Add(UMassStationaryISMSwitcherProcessor::StaticClass()->GetFName());

	UpdateParams.bTestCollisionAvailibilityForActorVisualization = false;
}

void UInstancedActorsVisualizationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) 
{
	Super::ConfigureQueries(EntityManager);

	EntityQuery.ClearTagRequirements(FMassTagBitSet(FMassVisualizationProcessorTag::StaticStruct()));
	EntityQuery.AddTagRequirement<FInstancedActorsVisualizationProcessorTag>(EMassFragmentPresence::All);
}
