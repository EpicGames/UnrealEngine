// Copyright Epic Games, Inc. All Rights Reserved.


#include "InstancedActorsVisualizationSwitcherProcessor.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsRepresentationSubsystem.h"
#include "MassCommonFragments.h"
#include "MassEntityQuery.h"
#include "MassExecutionContext.h"
#include "MassRepresentationFragments.h"
#include "MassRepresentationProcessor.h"
#include "MassRepresentationSubsystem.h"
#include "MassStationaryISMSwitcherProcessor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsVisualizationSwitcherProcessor)


UInstancedActorsVisualizationSwitcherProcessor::UInstancedActorsVisualizationSwitcherProcessor()
	: EntityQuery(*this)
{
	bAutoRegisterWithProcessingPhases = true;

	ExecutionOrder.ExecuteAfter.Add(UMassVisualizationProcessor::StaticClass()->GetFName());
	ExecutionOrder.ExecuteBefore.Add(UMassStationaryISMSwitcherProcessor::StaticClass()->GetFName());
}

void UInstancedActorsVisualizationSwitcherProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FInstancedActorsMeshSwitchFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassRepresentationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSharedRequirement<FMassRepresentationSubsystemSharedFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UInstancedActorsRepresentationSubsystem>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassRepresentationSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UInstancedActorsVisualizationSwitcherProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassRepresentationSubsystem* RepresentationSubsystem = Context.GetMutableSharedFragment<FMassRepresentationSubsystemSharedFragment>().RepresentationSubsystem;
		check(RepresentationSubsystem);
		checkSlow(RepresentationSubsystem->IsA<UInstancedActorsRepresentationSubsystem>());
		FMassInstancedStaticMeshInfoArrayView ISMInfosView = RepresentationSubsystem->GetMutableInstancedStaticMeshInfos();
		FMassInstancedSkinnedMeshInfoArrayView MeshInfosView = RepresentationSubsystem->GetMutableInstancedSkinnedMeshInfos();

		TConstArrayView<FInstancedActorsMeshSwitchFragment> MeshSwitchFragments = Context.GetFragmentView<FInstancedActorsMeshSwitchFragment>();
		TArrayView<FMassRepresentationFragment> RepresentationFragments = Context.GetMutableFragmentView<FMassRepresentationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassEntityHandle EntityHandle = Context.GetEntity(EntityIt);
			const FInstancedActorsMeshSwitchFragment& MeshSwitchFragment = MeshSwitchFragments[EntityIt];
			FMassRepresentationFragment& RepresentationFragment = RepresentationFragments[EntityIt];

			SwitchEntityMeshDesc(ISMInfosView, RepresentationFragment, EntityHandle, MeshSwitchFragment.NewStaticMeshDescHandle);
			SwitchEntitySkinnedMeshDesc(MeshInfosView, RepresentationFragment, EntityHandle, MeshSwitchFragment.NewSkinnedMeshDescHandle);

			Context.Defer().RemoveFragment<FInstancedActorsMeshSwitchFragment>(EntityHandle);
		}
	});
}

void UInstancedActorsVisualizationSwitcherProcessor::SwitchEntityMeshDesc(FMassInstancedStaticMeshInfoArrayView& ISMInfosView, FMassRepresentationFragment& RepresentationFragment, FMassEntityHandle EntityHandle, FStaticMeshInstanceVisualizationDescHandle NewStaticMeshDescHandle)
{
	if (NewStaticMeshDescHandle != RepresentationFragment.StaticMeshDescHandle)
	{
		// Remove current StaticMeshDescHandle ISMC instance before we switch to NewStaticMeshDescHandle
		// and 'forget' about it.
		if (RepresentationFragment.PrevRepresentation == EMassRepresentationType::StaticMeshInstance)
		{
			if (ensureMsgf(RepresentationFragment.StaticMeshDescHandle.IsValid(), TEXT("Switching visualization while StaticMeshDescHandle is invalid"))
				&& ensureMsgf(ISMInfosView.IsValidIndex(RepresentationFragment.StaticMeshDescHandle.ToIndex()), TEXT("Switching visualization while StaticMeshDescHandle is not a valid ISMInfo index, %d out of %d")
					, RepresentationFragment.StaticMeshDescHandle.ToIndex(), ISMInfosView.Num()))
			{
				const int32 ISMInfoIndex = RepresentationFragment.StaticMeshDescHandle.ToIndex();
				if (ensureMsgf(ISMInfosView.IsValidIndex(ISMInfoIndex), TEXT("Invalid handle index %u for ISMInfosView"), ISMInfoIndex))
				{
					FMassInstancedStaticMeshInfo& ISMInfo = ISMInfosView[ISMInfoIndex];

					// Note that we're using the PrevLODSignificance here, and the reason for it is that the Prev value matches the 
					// PrevRepresentation - thus we need to remove from the "previously" used LODSignificance range.
					ISMInfo.RemoveInstance(EntityHandle, RepresentationFragment.PrevLODSignificance);
				}
			}

			// Set PrevRepresentation to None to match the new removed instance state and let 
			// UMassStationaryISMSwitcherProcessor see that a new instance needs to be made
			RepresentationFragment.PrevRepresentation = EMassRepresentationType::None;
		}
		
		RepresentationFragment.StaticMeshDescHandle = NewStaticMeshDescHandle;
	}
}

void UInstancedActorsVisualizationSwitcherProcessor::SwitchEntitySkinnedMeshDesc(FMassInstancedSkinnedMeshInfoArrayView& MeshInfosView, FMassRepresentationFragment& RepresentationFragment, FMassEntityHandle EntityHandle, FSkinnedMeshInstanceVisualizationDescHandle NewMeshDescHandle)
{
	if (NewMeshDescHandle != RepresentationFragment.SkinnedMeshDescHandle)
	{
		// Remove current FSkinnedMeshInstanceVisualizationDescHandle Mesh Component instance before we switch to NewMeshDescHandle
		// and 'forget' about it.
		 if (RepresentationFragment.PrevRepresentation == EMassRepresentationType::SkinnedMeshInstance)
		{
			if (ensureMsgf(RepresentationFragment.SkinnedMeshDescHandle.IsValid(), TEXT("Switching visualization while MeshDescHandle is invalid"))
				&& ensureMsgf(MeshInfosView.IsValidIndex(RepresentationFragment.SkinnedMeshDescHandle.ToIndex()), TEXT("Switching visualization while MeshDescHandle is not a valid MeshInfo index, %d out of %d")
					, RepresentationFragment.SkinnedMeshDescHandle.ToIndex(), MeshInfosView.Num()))
			{
				const int32 MeshInfoIndex = RepresentationFragment.SkinnedMeshDescHandle.ToIndex();
				FMassInstancedSkinnedMeshInfo& MeshInfo = MeshInfosView[MeshInfoIndex];

				// Note that we're using the PrevLODSignificance here, and the reason for it is that the Prev value matches the 
				// PrevRepresentation - thus we need to remove from the "previously" used LODSignificance range.
				MeshInfo.RemoveInstance(EntityHandle, RepresentationFragment.PrevLODSignificance);
			}

			// Set PrevRepresentation to None to match the new removed instance state and let 
			// UMassStationaryISMSwitcherProcessor see that a new instance needs to be made
			RepresentationFragment.PrevRepresentation = EMassRepresentationType::None;
		}

		RepresentationFragment.SkinnedMeshDescHandle = NewMeshDescHandle;
	}
}
