// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsVisualizationTrait.h"
#include "InstancedActorsData.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsRepresentationActorManagement.h"
#include "InstancedActorsRepresentationSubsystem.h"
#include "InstancedActorsSettingsTypes.h"
#include "InstancedActorsTypes.h"
#include "InstancedActorsVisualizationProcessor.h"
#include "MassEntityTemplateRegistry.h"
#include "MassCommonFragments.h"
#include "MassActorSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InstancedActorsVisualizationTrait)


UInstancedActorsVisualizationTrait::UInstancedActorsVisualizationTrait(const FObjectInitializer& ObjectInitializer)
{
	Params.RepresentationActorManagementClass = UInstancedActorsRepresentationActorManagement::StaticClass();
	Params.LODRepresentation[EMassLOD::High] = EMassRepresentationType::HighResSpawnedActor;
	Params.LODRepresentation[EMassLOD::Medium] = EMassRepresentationType::HighResSpawnedActor;
	// @todo Re-enable this with proper handling of Enable / Disable 'kept' actors, in 
	// UInstancedActorsRepresentationActorManagement::SetActorEnabled, including replication to client.
	Params.bKeepLowResActors = false;

	bAllowServerSideVisualization = true;
	RepresentationSubsystemClass = UInstancedActorsRepresentationSubsystem::StaticClass();

	// Avoids registering the Static Mesh Descriptor during BuildTemplate, as it's already added
	// during UInstancedActorsManagers InitializeModifySpawn flow.
	bRegisterStaticMeshDesc = false;
}

void UInstancedActorsVisualizationTrait::InitializeFromInstanceData(UInstancedActorsData& InInstanceData)
{
	InstanceData = &InInstanceData;

	HighResTemplateActor = InstanceData->ActorClass;

	const bool bIsClient = InInstanceData.GetManagerChecked().IsNetMode(NM_Client);
	if (bIsClient)
	{
		// Don't attempt to spawn actors natively on clients. Instead, rely on bForceActorRepresentationForExternalActors to 
		// switch to Actor representation once replicated actors are set explicitly in UInstancedActorsData::SetReplicatedActor 
		// and maintain this until the actor is destroyed by the server, whereupon we'll fall back to StaticMeshInstance
		// again.
		Params.LODRepresentation[EMassLOD::High] = EMassRepresentationType::StaticMeshInstance;
		Params.LODRepresentation[EMassLOD::Medium] = EMassRepresentationType::StaticMeshInstance;
		Params.bForceActorRepresentationForExternalActors = true;
	}

	const FInstancedActorsSettings& Settings = InstanceData->GetSettings<const FInstancedActorsSettings>();
	
	StaticMeshInstanceDesc = InstanceData->GetDefaultVisualizationChecked().VisualizationDesc.ToMassVisualizationDesc();

	if (StaticMeshInstanceDesc.Meshes.IsEmpty())
	{
		ensure(InstanceData->GetDefaultVisualizationChecked().ISMComponents.IsEmpty());
		Params.LODRepresentation[EMassLOD::Low] = EMassRepresentationType::None;

		if (bIsClient)
		{
			// Don't attempt to switch to ISMC representation on clients for no mesh classes (which would otherwise crash)
			// Let the server spawn these actors and replicate them to clients.
			Params.LODRepresentation[EMassLOD::High] = EMassRepresentationType::None;
			Params.LODRepresentation[EMassLOD::Medium] = EMassRepresentationType::None;
		}
	}

	double SettingsMaxInstanceDistance = FInstancedActorsSettings::DefaultMaxInstanceDistance;
	float LODDistanceScale = 1.0f;
	double GlobalMaxInstanceDistanceScale = 1.0;
	Settings.ComputeLODDistanceData(SettingsMaxInstanceDistance, GlobalMaxInstanceDistanceScale, LODDistanceScale);

	LODParams.LODDistance[EMassLOD::High] = 0.f;
	LODParams.LODDistance[EMassLOD::Medium] = Settings.MaxActorDistance;
	LODParams.LODDistance[EMassLOD::Low] = Settings.MaxActorDistance;
	LODParams.LODDistance[EMassLOD::Off] = SettingsMaxInstanceDistance * GlobalMaxInstanceDistanceScale;
}

void UInstancedActorsVisualizationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	check(InstanceData.IsValid() || BuildContext.IsInspectingData());

	Super::BuildTemplate(BuildContext, World);

	// we need IAs to be processed by a dedicated visualization processor, configured a bit differently than the default one.
	BuildContext.RemoveTag<FMassVisualizationProcessorTag>();

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	BuildContext.AddFragment<FTransformFragment>();
	BuildContext.AddFragment<FMassActorFragment>();

	FInstancedActorsDataSharedFragment ManagerSharedFragment;
	ManagerSharedFragment.InstanceData = InstanceData;
	FSharedStruct SubsystemFragment = EntityManager.GetOrCreateSharedFragment(ManagerSharedFragment);

	FInstancedActorsDataSharedFragment* AsShared = SubsystemFragment.GetPtr<FInstancedActorsDataSharedFragment>();
	if (ensure(AsShared))
	{
		// this can happen when we unload data and then stream it back again - we end up with the same path object, but different pointer.
		// The old instance should be garbage now
		if (AsShared->bInstancedActorsDataDespawnedEntities || AsShared->InstanceData != InstanceData)
		{
			ensure(AsShared->bInstancedActorsDataDespawnedEntities || AsShared->InstanceData.IsValid() == false);
			AsShared->InstanceData = InstanceData;
		}

		AsShared->bInstancedActorsDataDespawnedEntities = false;
		
		// we also need to make sure the bulk LOD is reset since the shared fragment can survive the death of the original 
		// InstanceData while preserving the "runtime" value, which will mess up newly spawned entities.
		AsShared->BulkLOD = EInstancedActorsBulkLOD::MAX;

		if (BuildContext.IsInspectingData() == false)
		{
			InstanceData->SetSharedInstancedActorDataStruct(SubsystemFragment);
		}
	}
	// not adding SubsystemFragment do BuildContext on purpose, we temporarily use shared fragments to store IAD information.
	// To be moved to InstancedActorSubsystem in the future

	// ActorInstanceFragment will get initialized by UInstancedActorsInitializerProcessor
	BuildContext.AddFragment_GetRef<FMassActorInstanceFragment>();

	FInstancedActorsFragment& InstancedActorFragment = BuildContext.AddFragment_GetRef<FInstancedActorsFragment>();
	InstancedActorFragment.InstanceData = InstanceData;

	if (BuildContext.IsInspectingData() == false)
	{
		// @todo Implement version of AddVisualDescWithISMComponent that supports multiple ISMCs and use that here
		if (ensure(InstanceData.IsValid()))
		{
			FMassRepresentationFragment* RepresentationFragment = BuildContext.GetFragment<FMassRepresentationFragment>();
			if (ensureMsgf(RepresentationFragment, TEXT("Configuration error, we always expect to have a FMassRepresentationFragment instance at this point")))
			{
				RepresentationFragment->StaticMeshDescHandle = InstanceData->GetDefaultVisualizationChecked().MassStaticMeshDescHandle;

				if (RepresentationFragment->LowResTemplateActorIndex == INDEX_NONE)
				{
					// if there's no "low res actor" we reuse the high-res one, otherwise we risk the visualization actor getting 
					// removed when switching from EMassLOD::High down to EMassLOD::Medium
					RepresentationFragment->LowResTemplateActorIndex = RepresentationFragment->HighResTemplateActorIndex;
				}
			}
		}
	}
}
