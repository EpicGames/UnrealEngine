// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassActorSpawnerSubsystem.h"
#include "ClientInstancedActorsSpawnerSubsystem.generated.h"

#define UE_API INSTANCEDACTORS_API


/** 
 * Used on Clients to handle actor spawning synchronized with the Server. At the moment it boils down to storing
 * actor spawning requests and putting them in Pending state until the server-spawned actor gets replicated
 * over to the Client.
 */
UCLASS(MinimalAPI)
class UClientInstancedActorsSpawnerSubsystem : public UMassActorSpawnerSubsystem
{
	GENERATED_BODY()

protected:
	//~ Begin USubsystem Overrides
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	//~ Begin USubsystem Overrides

	//~ Begin UMassActorSpawnerSubsystem Overrides
	UE_API virtual ESpawnRequestStatus SpawnActor(const FSpawnArgs& Args, FActorSpawnParameters& InOutSpawnParameters) const override;
	UE_API virtual bool ReleaseActorToPool(AActor* Actor) override;
	//~ End UMassActorSpawnerSubsystem Overrides
};

#undef UE_API
