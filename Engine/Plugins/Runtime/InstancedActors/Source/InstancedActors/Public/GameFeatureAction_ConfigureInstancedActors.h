// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFeatureAction.h"
#include "InstancedActorsSettings.h"
#include "GameFeatureAction_ConfigureInstancedActors.generated.h"


/** 
 * GameplayFeature Action carrying overrides to InstancedActors settings
 */
UCLASS(meta = (DisplayName = "Configure InstancedActors"))
class UGameFeatureAction_ConfigureInstancedActors : public UGameFeatureAction
{
	GENERATED_BODY()

public:	
	//~ Begin UGameFeatureAction interface
	virtual void OnGameFeatureActivating(FGameFeatureActivatingContext& Context) override;
	virtual void OnGameFeatureDeactivating(FGameFeatureDeactivatingContext& Context) override;
	//~ End UGameFeatureAction interface

private:

	UPROPERTY(EditAnywhere, Category=InstancedActors)
	FInstancedActorsConfig ConfigOverride;
};

