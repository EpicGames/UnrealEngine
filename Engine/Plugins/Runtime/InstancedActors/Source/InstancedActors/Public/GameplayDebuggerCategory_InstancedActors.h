// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsDebug.h"
#include "MassEntityTypes.h"


#if WITH_GAMEPLAY_DEBUGGER && WITH_INSTANCEDACTORS_DEBUG && WITH_MASSENTITY_DEBUG

#include "GameplayDebuggerCategory.h"
#include "HAL/IConsoleManager.h"
#include "Mass/EntityHandle.h"

struct FMassEntityManager;

/** Gameplay debugger used to debug Intanced Actors */
class FGameplayDebuggerCategory_InstancedActors : public FGameplayDebuggerCategory
{
public:
	FGameplayDebuggerCategory_InstancedActors();
	~FGameplayDebuggerCategory_InstancedActors();

	void SetCachedEntity(const FMassEntityHandle Entity, const FMassEntityManager& EntityManager);
	void OnEntitySelected(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle);
	void ClearCachedEntity();

	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;

	INSTANCEDACTORS_API static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:

	static TArray<FAutoConsoleCommandWithWorld> ConsoleCommands;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnConsoleCommandBroadcastDelegate, UWorld*);
	static FOnConsoleCommandBroadcastDelegate OnToggleDebugLocalIAMsBroadcast; 	

	using FDelegateHandlePair = TPair<FOnConsoleCommandBroadcastDelegate*, FDelegateHandle>;
	TArray<FDelegateHandlePair> ConsoleCommandHandles;

	int32 ToggleDebugLocalIAMsInputIndex = INDEX_NONE;

	void OnToggleDebugLocalIAMs();

private:

	bool bDebugLocalIAMs = false;

	AActor* CachedDebugActor = nullptr;
	FMassEntityHandle CachedEntity;

	FDelegateHandle OnEntitySelectedHandle;
};

#endif // FN_WITH_GAMEPLAY_DEBUGGER && WITH_INSTANCEDACTORS_DEBUG && WITH_MASSENTITY_DEBUG
