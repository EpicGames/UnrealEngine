// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsManager.h"
#include "MassCommands.h"
#include "ProfilingDebugging/CpuProfilerTrace.h"

namespace UE::InstancedActors
{
	/**
	 * Returns the bitset indicating all the gate-tags of the processors we want to run on Detailed-LOD entities
	 * (i.e. not the Batched-LOD ones). These tags are switched by UInstancedActorsStationaryLODBatchProcessor
	 * Modifying the bitset is the way for project-specific code to influence what gets executed.
	 */
	INSTANCEDACTORS_API FMassTagBitSet& GetDetailedLODTags();

	struct FBulkLODTagsChangeCommandBase : public FMassCommandChangeTags
	{
		using FMassCommandChangeTags::FMassCommandChangeTags;
		using Super = FMassCommandChangeTags;
		void Add(TNotNull<UInstancedActorsData*> InstanceData, EInstancedActorsBulkLOD NewLOD);

	protected:
		virtual void Run(FMassEntityManager& EntityManager) override;
		virtual void Reset() override;
		mutable TArray<TPair<TWeakObjectPtr<UInstancedActorsData>, EInstancedActorsBulkLOD>> IADataToNotify;
	};

	/** Adds GetDetailedLODTags() to an entity, effectively enabling DetailedLOD processing on it */
	struct FEnableDetailedLODCommand : public FBulkLODTagsChangeCommandBase 
	{
		using Super = FBulkLODTagsChangeCommandBase;
		FEnableDetailedLODCommand();
	};

	/** Removes GetDetailedLODTags() from an entity, effectively enabling BatchLOD processing on it */
	struct FEnableBatchLODCommand : public FBulkLODTagsChangeCommandBase 
	{
		using Super = FBulkLODTagsChangeCommandBase;
		FEnableBatchLODCommand();
	};
}

/** 
 * Note: TManagerType is always expected to be AInstancedActorsManager, but is declared as 
 *	template's param to maintain uniform command adding interface via FMassCommandBuffer.PushCommand. 
 */
template<typename TManagerType, typename ... TOthers>
struct FMassCommandAddFragmentInstancesAndResaveIAPersistence : public FMassCommandAddFragmentInstances<TOthers...>
{
	using Super = FMassCommandAddFragmentInstances<TOthers...>;

	void Add(FMassEntityHandle Entity, typename TRemoveReference<TManagerType>::Type& ManagerToResave, TOthers... InFragments)
	{
		static_assert(TIsDerivedFrom<typename TRemoveReference<TManagerType>::Type, AInstancedActorsManager>::IsDerived, "TManagerType must be an AInstancedActorsManager");

		Super::Add(Entity, InFragments...);
		ManagersToResave.Add(&ManagerToResave);
	}

protected:
	virtual void Reset() override
	{
		ManagersToResave.Reset(); 
		Super::Reset();
	}

	virtual SIZE_T GetAllocatedSize() const override
	{
		return Super::GetAllocatedSize() + ManagersToResave.GetAllocatedSize();
	}

	virtual void Run(FMassEntityManager& EntityManager) override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(MassCommandAddFragmentInstancesAndResaveIAPersistence_Execute);

		// Add / update fragments
		Super::Run(EntityManager);

		// Resave Instanced Actor persistence for now-updated fragments
		for (const TObjectPtr<AInstancedActorsManager>& ManagerToResave : ManagersToResave)
		{
			if (IsValid(ManagerToResave))
			{
				ManagerToResave->RequestPersistentDataSave();
			}
		}
	}

	TSet<TObjectPtr<AInstancedActorsManager>> ManagersToResave;
};
