// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsCommands.h"
#include "InstancedActorsDebug.h"
#include "InstancedActorsTypes.h"
#include "Components/ActorComponent.h"
#include "MassCommandBuffer.h"
#include "MassEntityManager.h"
#include "MassEntityTypes.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/SharedPointer.h"
#include "InstancedActorsComponent.generated.h"

#define UE_API INSTANCEDACTORS_API


class UInstancedActorsData;
class UMassEntityTraitBase;
struct FMassEntityConfig;
struct FMassEntityTemplateData;

/** 
 * Provides Mass Entity reference and interop functions for Actors spawned via Instanced Actors on both client & server.
 */ 
UCLASS(MinimalAPI, ClassGroup="Instanced Actors", Meta=(BlueprintSpawnableComponent))
class UInstancedActorsComponent : public UActorComponent
{
	GENERATED_BODY()

public:

	UE_API UInstancedActorsComponent();

	/** 
	 * Called on servers before InitializeComponent & BeginPlay in UServerInstancedActorsSpawnerSubsystem::SpawnActor for 
	 * Actors spawned by the Instanced Actor system. 
     * Note: This is *not* called for BP constructed components which aren't added until after PreSpawnInit.
	 */
	UE_API virtual void OnServerPreSpawnInitForInstance(FInstancedActorsInstanceHandle InInstanceHandle);

	/** 
	 * Called on servers during InitializeComponent for Actors spawned by the Instanced Actor system. 
	 * Note: This *is* called for both native *and* BP contructed components, as opposed to OnServerPreSpawnInitForInstance which is only
	 * called for natively added components.
	 */
	UE_API virtual void InitializeComponentForInstance(FInstancedActorsInstanceHandle InInstanceHandle);

	/** 
	 * Called on an 'exemplar' Actor's components for clients & servers during UInstancedActorsData::CreateEntityTemplate to provide 
	 * UInstancedActorsComponent's an opportunity to extend Mass entity default traits.
	 * 
	 * Note: The `exemplar' actor is an actor spawned into a separate inactive UWorld by UInstancedActorsSubsystem::GetOrCreateExemplarActor
	 * for data mining like this.
	 *
	 * @param InMassEntityManager	The MassEntityManager to use for shared fragment registration etc
	 * @param InstancedActorData	The instance data the mass entity config will be used to spawn entities for. This component will be a default 
	 *								constructed component in InstancedActorData.ActorClass.
	 * @param InOutMassEntityConfig The Mass Entity Config to modify, e.g: via InOutMassEntityConfig.AddTrait
	 */
	virtual void ModifyMassEntityConfig(FMassEntityManager& InMassEntityManager, UInstancedActorsData* InstancedActorData, FMassEntityConfig& InOutMassEntityConfig) const {}

	/** 
	 * Called on ActorClass CDO components for clients & servers during UInstancedActorsData::CreateEntityTemplate to provide 
	 * UInstancedActorsComponent's an opportunity to extend Mass entity default fragments.
	 *
	 * Called after ModifyMassEntityConfig, once the entity config has been resolved to a template using GetOrCreateEntityTemplate and
	 * copied to a transient FMassEntityTemplateData (InOutMassEntityTemplateData) for further modification.
	 *
	 * @param InMassEntityManager			The MassEntityManager to use for shared fragment registration etc
	 * @param InstancedActorData			The instance data the mass entity config will be used to spawn entities for. This component will be a default 
	 *        								constructed component in InstancedActorData.ActorClass.
	 * @param InOutMassEntityTemplateData	The Mass Entity Template to modify, e.g: via InOutMassEntityTemplateData.AddFragment etc 
	 */
	virtual void ModifyMassEntityTemplate(FMassEntityManager& InMassEntityManager, UInstancedActorsData* InstancedActorData, FMassEntityTemplateData& InOutMassEntityTemplateData) const {}

	/**
	 * Subclasses implementing SerializeInstancePersistenceData must implement this method and return a non-zero unique uint32 used to 
	 * match serialized persistence records with this UInstancedActorsComponent's SerializeInstancePersistenceData implementation.
	 */
	virtual uint32 GetInstancePersistenceDataID() const { return 0; }

	/**
	 * Called prior to SerializeInstancePersistenceData for both saving & loading persistence data, to check if we indeed have persistence data to write / want to read.
	 * @return true if SerializeInstancePersistenceData should be called. If false, no entry will be written.
	 */
	virtual bool ShouldSerializeInstancePersistenceData(const FArchive& Archive, UInstancedActorsData* InstanceData, int64 TimeDelta) const { return false; }

	/**
	 * Called by AInstancedActorsManager::SerializeInstancePersistenceData for IAD's with an ActorClass containing this UInstancedActorsComponent,
	 * to save / load extended persistence data.
	 *
	 * Note: This is only called if ShouldSerializeInstancePersistenceData returns true, in which case GetInstancePersistenceDataID must also return a non-zero ID.
	 *
	 * @param Record			The archive record to read / write IAD save data to
	 * @param InstanceData		The InstanceData to serialize from / to
	 * @param TimeDelta	Real time in seconds since serialization (0 when saving)
	 *	
	 * @see ShouldSerializeInstancePersistenceData, GetInstancePersistenceDataID
	 */
	virtual void SerializeInstancePersistenceData(FStructuredArchive::FRecord Record, UInstancedActorsData* InstanceData, int64 TimeDelta) const {}

	/** 
	 * Returns true if this Actor was spawned by the Instanced Actor system for a Mass Entity.
	 * On servers, MassEntityHandle will be set in UInstancedActorsComponent::InitializeComponent (before component BeginPlay)
	 * On clients, MassEntityHandle will be set after component BeginPlay, as the link between Entity <-> Actor isn't made until
	 * both the actor has replicated to the client (recieving BeginPlay in the process) *and* the client Mass simulation attempts
	 * to spawn an actor itself, whereupon the replicated actor is then returned for the client spawn request.
	 */
	UFUNCTION(BlueprintPure, Category=InstancedActors)
	bool HasMassEntity() const { return GetMassEntityHandle().IsValid(); }

	/** 
	 * Handle to this Instanced Actor's Mass Entity, if this components Actor owner was spawned by the Instanced Actor system, invalid handle otherwise.
	 * On servers, MassEntityHandle will be set in UInstancedActorsComponent::InitializeComponent (before component BeginPlay)
	 * On clients, MassEntityHandle will be set after component BeginPlay.
	 */
	UE_API FMassEntityHandle GetMassEntityHandle() const;

	/** If HasMassEntity(), the Mass Entity Manager for MassEntityHandle */
	UE_API TSharedPtr<FMassEntityManager> GetMassEntityManager() const;

	UE_API FMassEntityManager& GetMassEntityManagerChecked() const;

	/** 
	 * If HasMassEntity(), adds a new default constructed FragmentType to this Instanced Actor's Mass entity, if not already 
	 * present (no-op otherwise)
	 * Performed deferred using FMassCommandBuffer to safely add fragments, as Mass may currently be performing parallel processing.
	 */
	template<typename FragmentType>
	void AddFragmentDeferred();

	/** 
	 * If HasMassEntity(), adds or updates existing FragmentTypes for this Instanced Actor's Mass entity.
	 * Performed deferred using FMassCommandBuffer to safely add fragments, as Mass may currently be performing parallel processing.
	 */
	template<typename... FragmentTypes>
	void AddOrUpdateFragmentsDeferred(FragmentTypes&&... NewFragments);

	/** 
	 * If HasMassEntity(), adds or updates existing FragmentTypes for this Instanced Actor's Mass entity, then requests a
	 * persistent data resave once the fragments are updated.
	 * Performed deferred using FMassCommandBuffer to safely add fragments, as Mass may currently be performing parallel processing.
	 * Note: The fragments are not automatically persisted (yet), it is assumed some other mechanism will read the new fragment values 
	 *       during the persistence resave, e.g: SerializeInstancePersistenceData
	 */
	template<typename... FragmentTypes>
	void AddOrUpdatePersistentFragmentsDeferred(FragmentTypes&&... NewFragments);

	/** 
	 * If HasMassEntity(), returns existing FragmentType fragment for this Instanced Actor's Mass Entity, if any. Otherwise returns nullptr.
	 * 
	 * Note: This is a synchronous call and whilst the returned fragment is guaranteed to be a valid pointer, the fragment may be getting written 
	 * to by Mass processors on parallel threads, unless they have bRequiresGameThreadExecution = true. Thread-safety in this scenario is left to
	 * the caller to either implement thread locking or ensure *all* processors reading and writing to this fragment have bRequiresGameThreadExecution = true.
	 */
	template<typename FragmentType>
	const FragmentType* GetVolatileFragment() const;

	/** 
	 * If HasMassEntity(), removes FragmentType from this Instanced Actor's Mass entity, if present (no-op otherwise)
	 * Performed deferred using FMassCommandBuffer to safely add fragments, as Mass may currently be performing parallel processing.
	 */
	template<typename FragmentType>
	void RemoveFragmentDeferred();

protected:

	/** 
	* Handle to the instance that spawned this Instanced Actor, if this component's Actor owner was spawned by the Instanced Actor 
	* system, invalid handle otherwise.
	* On servers, InstanceHandle will be set in UInstancedActorsComponent::InitializeComponent (before component BeginPlay)
	* On clients, InstanceHandle will be set after component BeginPlay.
	*/
	UPROPERTY(ReplicatedUsing=OnRep_InstanceHandle, Transient, Meta=(AllowPrivateAccess="true"), BlueprintReadOnly, Category=InstancedActors)
	FInstancedActorsInstanceHandle InstanceHandle;

	UFUNCTION()
	UE_API void OnRep_InstanceHandle();

	//~ Begin UObject Overrides
	UE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	//~ Begin UObject Overrides

	//~ Begin AActorComponent Overrides
	UE_API virtual void InitializeComponent() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	//~ End AActorComponent Overrides
};


template<typename FragmentType>
void UInstancedActorsComponent::AddFragmentDeferred()
{
	FMassEntityHandle MassEntityHandle = GetMassEntityHandle();
	if (ensure(MassEntityHandle.IsValid()))
	{
		GetMassEntityManagerChecked().Defer().AddFragment<FragmentType>(MassEntityHandle);
	}
}

template<typename... FragmentTypes>
void UInstancedActorsComponent::AddOrUpdateFragmentsDeferred(FragmentTypes&&... NewFragments)
{
	FMassEntityHandle MassEntityHandle = GetMassEntityHandle();
	if (ensure(MassEntityHandle.IsValid()))
	{
		GetMassEntityManagerChecked().Defer().PushCommand<FMassCommandAddFragmentInstances>(MassEntityHandle, Forward<FragmentTypes>(NewFragments)...);
	}
}

template<typename... FragmentTypes>
void UInstancedActorsComponent::AddOrUpdatePersistentFragmentsDeferred(FragmentTypes&&... NewFragments)
{
	FMassEntityHandle MassEntityHandle = GetMassEntityHandle();
	if (ensure(MassEntityHandle.IsValid()))
	{
		GetMassEntityManagerChecked().Defer().PushCommand<FMassCommandAddFragmentInstancesAndResaveIAPersistence>(MassEntityHandle, InstanceHandle.GetManagerChecked(), Forward<FragmentTypes>(NewFragments)...);
	}
}

template<typename FragmentType>
const FragmentType* UInstancedActorsComponent::GetVolatileFragment() const
{
	FMassEntityHandle MassEntityHandle = GetMassEntityHandle();
	if (ensure(MassEntityHandle.IsValid()))
	{
		return GetMassEntityManagerChecked().GetFragmentDataPtr<FragmentType>(MassEntityHandle);
	}

	return nullptr;
}

template<typename FragmentType>
void UInstancedActorsComponent::RemoveFragmentDeferred()
{
	FMassEntityHandle MassEntityHandle = GetMassEntityHandle();
	if (ensure(MassEntityHandle.IsValid()))
	{
		GetMassEntityManagerChecked().Defer().RemoveFragment<FragmentType>(MassEntityHandle);
	}
}

#undef UE_API
