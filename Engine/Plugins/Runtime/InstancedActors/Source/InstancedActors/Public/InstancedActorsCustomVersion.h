// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"

#define UE_API INSTANCEDACTORS_API

struct FInstancedActorsCustomVersion
{
	enum Type
	{
		// Before any version changes were made in the plugin
		InitialVersion = 0,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	UE_API const static FGuid GUID;

private:
	FInstancedActorsCustomVersion() {}
};

#undef UE_API
