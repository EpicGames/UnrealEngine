// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsTypes.h"
#include "InstancedActorsReplication.h"
#include "MassEntityConfigAsset.h"
#include "MassEntityTemplate.h"
#include "HAL/CriticalSection.h"

#include "InstancedActorsData.generated.h"

#define UE_API INSTANCEDACTORS_API

class AInstancedActorsManager;
struct FInstancedActorsSettings;

namespace UE::InstancedActors
{
	namespace CVars
	{
		extern bool bFlagSharedFragmentInsteadOfNullify;
	}
}

namespace UE::InstancedActors
{
struct FExemplarActorData
{
	~FExemplarActorData();

	TObjectPtr<AActor> Actor;
private:
	FExemplarActorData(AActor* InActor, UInstancedActorsSubsystem* InInstancedActorSubsystem)
		: Actor(InActor)
		, InstancedActorsSubsystem(InInstancedActorSubsystem)
	{
	}

	UInstancedActorsSubsystem* InstancedActorsSubsystem;

	friend class ::UInstancedActorsSubsystem;
};
} // UE::InstancedActors

// @todo there's a lot of public variables in this class, and properties are mixed with functions. A refactor is coming soon.

/**
 * Instance data for all instances of a given AActor class.
 * Provides 'stable' referral to instances by index via offline population of InstanceTransforms which is then
 * consistently loaded on both client and server.
 * @see AInstancedActorsManager
 */
UCLASS(MinimalAPI, DefaultToInstanced)
class UInstancedActorsData : public UObject
{
	GENERATED_BODY()

public:
	friend AInstancedActorsManager;

	// Called early in AInstancedActorsManager::InitializeModifyAndSpawnEntities to intitalize Settings, default visualization & Mass entity template
	UE_API void Initialize();

	// Called in AInstancedActorsManager::InitializeModifyAndSpawnEntities to spawn Mass entities for each instance
	UE_API void SpawnEntities();
	// Called early in AInstancedActorsManager::EndPlay to reconstruct cooked data state from runtime Mass entities as best we can,
	// then despawn all Mass entities and reset any other runtime instance data.
	// Note: does NOT nullify the shared fragment, that is done in Deinitialize(), which is only called on true teardown
	// and not on subsystem switch, allowing the fragment to be reused by an immediate respawn.
	UE_API void DespawnEntities();

	UE_DEPRECATED(5.8, "Using DespawnEntities with a parameter is deprecated.")
	UE_API void DespawnEntities(bool bIsSwitchingSubsystems);

	// Called from the parent's AInstancedActorsManager::EndPlay. Nullifies the shared fragment and can release
	// the entity template and exemplar Actor from memory.
	UE_API void Deinitialize();

	UFUNCTION(BlueprintPure, Category = InstancedActors)
	UE_API AInstancedActorsManager* GetManager() const;

	UE_API AInstancedActorsManager& GetManagerChecked() const;

	// Returns the 'owning' UInstancedActorsData for EntityHandle by checking it's FInstancedActorsFragment (if any)
	// returning nullptr for unknown entities.
	static UE_API UInstancedActorsData* GetInstanceDataForEntity(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle);

	UE_API FInstancedActorsInstanceIndex GetInstanceIndexForEntity(const FMassEntityHandle EntityHandle) const;

	UE_API FMassEntityHandle GetEntity(FInstancedActorsInstanceIndex InstanceIndex) const;

	UE_API void SetSharedInstancedActorDataStruct(FSharedStruct InSharedStruct);
	
	/** @return current Bulk LOD of this InstancedActorData instance */
	EInstancedActorsBulkLOD GetBulkLOD() const
	{
		if (const FInstancedActorsDataSharedFragment* DataFragment = SharedInstancedActorDataStruct.GetPtr<FInstancedActorsDataSharedFragment>())
		{
			return DataFragment->BulkLOD;
		}
		ensureMsgf(bHasEverInitialized == false, TEXT("We expect to hit this path only if the IAD instance has not been initialized yet."));
		return EInstancedActorsBulkLOD::MAX;
	}

	// Identifying integer, unique within outer IAM, used for identifying matching persistence records
	// Incrementally assigned in GetOrCreateActorInstanceData
	UPROPERTY(VisibleAnywhere, Category=InstancedActors)
	uint16 ID;

	// The fully realized actor type for these instances
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=InstancedActors)
	TSubclassOf<AActor> ActorClass;

	const FInstancedActorsTagSet& GetAdditionalTags() const
	{
		return AdditionalTags;
	}

	template<typename T>
	const std::remove_const_t<T>* GetSettingsPtr() const
	{
		return SharedSettings.GetPtr<const std::remove_const_t<T>>();
	}

	template<typename T>
	const std::remove_const_t<T>& GetSettings() const
	{
		return SharedSettings.Get<const std::remove_const_t<T>>();
	}

	void SetSharedSettings(const FConstSharedStruct& InSharedSettings)
	{
		SharedSettings = InSharedSettings;
	}

	// This event virtual is public because it is called by InstancedActorsStationaryLODBatchProcessor
	virtual void OnBulkLODChanged(EInstancedActorsBulkLOD NewBulkLOD)
	{
	}

protected:

	virtual void OnInitialize()
	{
	}

	virtual void OnSpawnEntities()
	{
	}

	virtual void OnDespawnEntities()
	{
	}

	virtual void OnDeinitialize()
	{
	}

#if WITH_EDITOR
	// Called from GetOrCreateActorInstanceData after EditorPreviewISMComponents are first created,
	// allowing subclasses to customize them (e.g. apply material overrides).
	virtual void OnEditorPreviewISMCsCreated()
	{
	}
#endif

	// Delimiting tag set to group sets of instances
	UPROPERTY(VisibleAnywhere, Category=InstancedActors)
	FInstancedActorsTagSet AdditionalTags;
	
	// Compiled settings for these instances, compiled and cached in BeginPlay.
	// Note: Can also be refreshed on demand in PIE, using IA.RefreshSettings CVar
	FConstSharedStruct SharedSettings;
	/*
	@todo note that at the moment Settings do not contain UObjects so we don't need to track the references. But it's a good
	idea to support UObject properties in settings, so we'll add something like the folowing soon:

		void UInstancedActorsData::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
		{
			Super::AddReferencedObjects(InThis, Collector);

			const UInstancedActorsData* Data = Cast<UInstancedActorsData>(InThis);
			check(Data);
		  if (SharedSettings.IsValid())
		  {
			Collector.AddPropertyReferencesWithStructARO(SharedSettings->GetScriptStruct(), SharedSettings->GetMemory(), Data);
		  }
		}	
	*/

public:
	// Per-instance transforms. This essentially forms the 'instance list' until Entities are spawned from this.
	// Note: This list can contain invalid 'removed' entries with 0'd values which will be skipped by
	//       UInstancedActorsInitializerProcessor
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category=InstancedActors)
	TArray<FTransform> InstanceTransforms;

	// InstanceTransforms.Num() - number of invalid transforms
	// Note: This is only valid prior to Entity spawning when InstanceTransforms is also valid, after which, spawned
	// entities may be later removed / culled at runtime, invalidating this number.
	// @todo Bundle this and InstanceTransforms, Bounds etc into a `SourceData` to separate them from runtime entity data
	// @see InstanceTransforms
	UPROPERTY()
	uint16 NumValidInstances = 0;

	// Cumulative mesh bounds for all of InstanceTransfroms
	UPROPERTY()
	FBox Bounds = FBox(ForceInit);

	// InstanceTransforms.Num() cached in PostLoad so we can restore InstanceTransforms to this
	// size in ResetInstanceData
	UPROPERTY(Transient)
	uint16 NumInstances = 0;

	// Runtime spawned mass instances
	// @see AInstancedActorsManager::SpawnInstances
	UPROPERTY(Transient)
	TArray<FMassEntityHandle> Entities;

	// The entity template to spawn Mass entities from
	// @see CreateEntityTemplate
	FMassEntityTemplateID EntityTemplateID;

	// Distance from where the object will be culled out - auto computed from bounding box radius or can be set per via the settings
	float MaxDrawDistance = FLT_MAX;

	// Distance from where the lowest LOD will be rendered - auto computed from bounding box and screen size data stored in the StaticMesh or can be set via the settings
	float LowLODDrawDistance = FLT_MAX;

	// Returns true if InstanceTransforms has been consumed to spawn Mass entities
	UE_API bool HasSpawnedEntities() const;

	bool CanHydrate() const;

	// Returns the total instance count, including both valid & invalid instances e.g: GetNumFreeInstances() + NumValidInstances
	// Useful for iterating by FInstancedActorsInstanceIndex or allocating same-sized instance data arrays
	UE_API int32 GetNumInstances() const;

	// Returns the current invalid instance count, if any (i.e: GetNumInstances() - NumValidInstances)
	UE_API int32 GetNumFreeInstances() const;

	// Returns true if InstanceHandle refers to this instance data and we have current information for an
	// instance at InstanceHandle.InstanceIndex
	UE_API bool IsValidInstance(const FInstancedActorsInstanceHandle& InstanceHandle) const;
	
	// Performs setup after all Instances have been loaded. Canonically called from PostLoad(), but may need to be called manually
	// if this UInstancedActorsData is created at cook/runtime
	UE_API void SetupLoadedInstances();

#if WITH_EDITOR
	// Add's an instance of this actor type at Transform
	// @see AInstancedActorsManager::AddActorInstance
	UE_API FInstancedActorsInstanceHandle AddInstance(const FTransform& Transform, const bool bWorldSpace = false);

	// Set/update the transform for an instance.
	//
	UFUNCTION(BlueprintCallable, Category = InstancedActors)
	UE_API bool SetInstanceTransform(const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& Transform, const bool bWorldSpace);

	// Removes all instance data for InstanceHandle.
	//
	// This simply adds this instance to a FreeList which incurs extra cost to process at runtime before instance spawning.
	//
	// Note: By default UInstancedActorsSubsystem::RemoveActorInstance will destroy empty managers on
	// last instance removal, implicitly clearing this for subsequent instance adds which would create
	// a fresh manager.
	//
	// @see IA.CompactInstances console command
	UE_API bool RemoveInstance(const FInstancedActorsInstanceHandle& InstanceToRemove);

	// Iterates all ISMCs created in GetOrCreateActorInstanceData.
	// @param InFunction						The function to call for each ISMC.
	UE_API void ForEachEditorPreviewISMC(TFunctionRef<bool(UInstancedStaticMeshComponent& /*ISMComponent*/)> InFunction) const;
#endif

	// Removes RuntimeRemoveInstances as if they were never present i.e: these removals are not persisted as
	// if made by a player.
	// Prior to entity spawning this simply invalidates InstanceTransforms entries, post entity spawning this
	// destroys spawned entities.
	//
	// Note: Any RuntimeRemoveInstances that have already been removed are safely skipped.
	UE_API void RuntimeRemoveInstances(TConstArrayView<FInstancedActorsInstanceIndex> RuntimeRemoveInstances);

	// Removes all instances as if they were never present i.e: these removals are not persisted as
	// if made by a player.
	// Prior to entity spawning this simply invalidates InstanceTransforms entries, post entity spawning this
	// destroys spawned entities.
	//
	// Note: Any RuntimeRemoveInstances that have already been removed are safely skipped.
	UE_API void RuntimeRemoveAllInstances();

	// Authority: Permanently destroy InstanceToDestroy by adding it to the persistent DestroyedInstances list
	// and deleting it's Mass entity (deleting any Mass-spawned actors in turn).
	// DestroyedInstances modifications are then replicated to clients.
	// Non-authority: 'Predict' replicated destruction by removing the instance to prevent ISMC instance addition
	// once actor destruction completes.
	UE_API void DestroyInstance(FInstancedActorsInstanceIndex InstanceToDestroy);

	// Authority: `Eject` an instance by unlinking it's Actor from Mass, setting the actor to persist itself and
	// then destroying the manager instance. This effectively transfers the actor to the persistence
	// system.
	// Non-authority: Perform the same unlinking and entity removal locally to release the Actor from Mass.
	UE_API virtual void EjectInstanceActor(FInstancedActorsInstanceIndex InstanceToEject, AActor& ActorToEject);

	// Called in UInstancedActorsComponent::OnRep_InstanceHandle on clients to set ReplicatedActor as the current
	// actor for Instance's mass entity. As we set bForceActorRepresentationWhileAvailable in UInstancedActorsVisualizationTrait
	// this then forces the entity to switch to Actor representation on clients.
	UE_API void SetReplicatedActor(FInstancedActorsInstanceIndex Instance, AActor& ReplicatedActor);

	// Called in UInstancedActorsComponent::EndPlay on clients to clear / unset the current
	// actor for Instance's mass entity.
	// As we set bForceActorRepresentationWhileAvailable in UInstancedActorsVisualizationTrait
	// this then allows the natural WantedRepresentationType (ISMC) to restore
	UE_API void ClearReplicatedActor(FInstancedActorsInstanceIndex Instance, AActor& ExpectedActor);

	// Returns the default visualization auto-created in BeginPlay from ActorClass and used
	// by default for all spawned entities in BeginPlay.
	const FInstancedActorsVisualizationInfo& GetDefaultVisualizationChecked() const { return GetVisualizationChecked(0); }

	// Get runtime visualization info by index for visualizations added with AddVisualization.
	// Asserts if VisualizationIndex doesn't correspond to a valid index, or the visualization at that index is
	// invalid (has been removed)
	// @param VisualizationIndex The index of the visualization previously returned by AddVisualization
	const FInstancedActorsVisualizationInfo& GetVisualizationChecked(uint8 VisualizationIndex) const
	{
		check(InstanceVisualizations.IsValidIndex(VisualizationIndex));
		check(InstanceVisualizationAllocationFlags.IsValidIndex(VisualizationIndex));
		const FInstancedActorsVisualizationInfo& VisualizationInfo = InstanceVisualizations[VisualizationIndex];
		check(InstanceVisualizationAllocationFlags[VisualizationIndex] == true);
		return VisualizationInfo;
	}

	// Get runtime visualization info by index for visualizations added with AddVisualization.
	// If VisualizationIndex doesn't correspond to a valid index, or the visualization at that index is
	// invalid (has been removed) nullptr is returned.
	// @param VisualizationIndex The index of the visualization previously returned by AddVisualization
	const FInstancedActorsVisualizationInfo* GetVisualization(uint8 VisualizationIndex) const
	{
		if (InstanceVisualizations.IsValidIndex(VisualizationIndex))
		{
			check(InstanceVisualizationAllocationFlags.IsValidIndex(VisualizationIndex));
			if (InstanceVisualizationAllocationFlags[VisualizationIndex] == true)
			{
				return &InstanceVisualizations[VisualizationIndex];
			}
		}

		return nullptr;
	}

	// Register additional / alternate VisualizationDesc for instances to switch to, creating ISMC's
	// for each VisualizationDesc.InstancedMeshes
	// @warning No more than 254 visualizations may be registered at any time to allow for uint8 indexing.
	// @return Index handle to refer to the registered visualization
	UE_API uint8 AddVisualization(FInstancedActorsVisualizationDesc& InOutVisualizationDesc);

	// Register additional / alternate VisualizationDesc for instances to switch to, creating ISMC's
	// for each VisualizationDesc.InstancedMeshes
	// @warning No more than 254 visualizations may be registered at any time to allow for uint8 indexing.
	// @return Index handle to refer to the registered visualization
	UE_API uint8 AddVisualizationAsync(const FInstancedActorsSoftVisualizationDesc& SoftVisualizationDesc);

	// Iterates all currently 'allocated' visualizations previously added with AddVisualization or AddVisualizationAsync
	// @param InFunction						The function to call for each allocated visualization.
	// @param bSkipAsyncLoadingVisualizations	If true, skips visualizations that are still async loading
	//											(FInstancedActorsVisualizationInfo::IsAsyncLoading() = true) which won't have been
	//											initialized yet i.e: won't yet have a valid Desc, ISMComponents or MassStaticMeshDescIndex
	//											yet.
	UE_API void ForEachVisualization(TFunctionRef<bool(uint8 /*VisualizationIndex*/, const FInstancedActorsVisualizationInfo&  /*Visualization*/)> InFunction
		, bool bSkipAsyncLoadingVisualizations = true) const;

	// Switches InstanceToSwitch to use the 'visualization' at NewVisualizationIndex, previoulsy added via
	// AddVisualzation. ISMC instances will be removed from the current /former visualization for these instances
	// and new instances will be added to the ISMC's of the new visualization.
	//
	// Note: This is performed via a mass deferred command to add a 'pending' FInstancedActorsMeshSwitchFragment
	// to the instances entity. UInstancedActorsVisualizationSwitcherProcessor will then process this switch
	// fragment to remove the current visualization's ISMC instances if any and setup NewVisualizationIndex
	// to be instanced instead.
	UE_API void SwitchInstanceVisualization(FInstancedActorsInstanceIndex InstanceToSwitch, uint8 NewVisualizationIndex);

	// Remove previously registered instance visualization, deleting all ISMC's
	UE_API void RemoveVisualization(uint8 VisualizationIndex);

	// Remove all previously registered instance visualization, deleting all ISMC's
	UE_API void RemoveAllVisualizations();

	// Called by UInstancedActorsRepresentationActorManagement when a managed actor is destroyed
	UE_API void OnInstancedActorDestroyed(AActor& DestroyedActor, const FMassEntityHandle EntityHandle);

	// Called by UInstancedActorsRepresentationActorManagement when a managed actor is moved
	// @return true if MovedActor was ejected
	UE_API bool OnInstancedActorMoved(AActor& MovedActor, const FMassEntityHandle EntityHandle);

	// Called when persistent data has been applied / restored
	UE_API void OnPersistentDataRestored();

	inline FInstancedActorsDeltaList& GetMutableInstanceDeltaList() { return InstanceDeltas; }

	inline const FInstancedActorsDeltaList& GetInstanceDeltaList() const { return InstanceDeltas; }

	// Called by FInstancedActorsDeltaList::PostReplicatedAdd and PostReplicatedChanged on InstanceDelta replication
	UE_API void OnRep_InstanceDeltas(TConstArrayView<int32> UpdatedInstanceDeltaIndices);

	// Called by FInstancedActorsDeltaList::PreReplicatedRemove on InstanceDelta removal replication (just before the actual array element removal)
	UE_API void OnRep_PreRemoveInstanceDeltas(TConstArrayView<int32> RemovedInstanceDeltaIndices);

	// Called on both server and client to apply instance delta changes to mass entities
	// On servers: Called by OnPersistentDataRestored after persistence record has deserialized the delta data
	// On clients: Called by OnRep_InstanceDeltas when new delta data has replicated from the server
	// @param InstanceDeltaIndices Indices into InsanceDeltas.GetInstanceDeltas() of the upated deltas to apply
	UE_API void ApplyInstanceDeltas();
	UE_API void ApplyInstanceDeltas(TConstArrayView<int32> InstanceDeltaIndices);

	// Called on clients by OnRep_PreRemoveInstanceDeltas to revert instance delta change to mass entities
	// @param InstanceDeltaIndices Indices into InsanceDeltas.GetInstanceDeltas() of the deltas to revert
	UE_API void RollbackInstanceDeltas(TConstArrayView<int32> InstanceDeltaIndices);

	// Returns a string useful for identifying this instance data object withing it's owning manager
	// @param bCompact if true, this IAD's pointer is used, otherwise the full manager object path is included
	UE_API FString GetDebugName(bool bCompact = false) const;

	UE_API virtual void UpdateCullDistance();

	// mz@todo IA: move this section back
	// Server-only. Used by LifecycleComponent & LifecycleProcessor to replicate & persist lifecycle changes
	// @todo Provide generic fragment persistence & replication
	UE_API void SetInstanceCurrentLifecyclePhase(FInstancedActorsInstanceIndex InstanceIndex, uint8 InCurrentLifecyclePhaseIndex);

	// mz@todo IA: move this section back
	// Server-only. Used by LifecycleComponent & LifecycleProcessor to remove replicated & persisted lifecycle
	// changes (triggering a rollback to default values on clients)
	// @todo Provide generic fragment persistence & replication
	UE_API void RemoveInstanceLifecyclePhaseDelta(FInstancedActorsInstanceIndex InstanceIndex);

	// mz@todo IA: move this section back
	// Server-only. Used by LifecycleComponent & LifecycleProcessor to remove the lifecycle time elapsed
	// delta record for InstanceIndex from InstanceDeltas (if any)
	// @todo Provide generic fragment persistence & replication
	UE_API void RemoveInstanceLifecyclePhaseTimeElapsedDelta(FInstancedActorsInstanceIndex InstanceIndex);

	int32 GetInstanceDataID() const { return (int32)ID; }

	const FBox& GetCachedLocalBounds() const { return CachedLocalBounds; }

	FMassEntityHandle GetEntityHandleForIndex(const FInstancedActorsInstanceIndex Index) const
	{
		return Entities.IsValidIndex(Index.GetIndex()) ? Entities[Index.GetIndex()] : FMassEntityHandle();
	}

	UE_API FMassEntityManager& GetMassEntityManagerChecked() const;

	UE_API int32 GetEntityIndexFromCollisionIndex(const UInstancedStaticMeshComponent& ISMComponent, const int32 CollisionIndex) const;

	const FGameplayTagContainer& GetCombinedTags() const { return CombinedTags; }

	/**
	 * Moves the current list of entities pending destruction (added via QueueEntitiesForDestruction) to InOutEntities
	 * This 'consumes' these pending entities with the assumption they will get destroyed by next command flush.
	 * @see UInstancedActorsRuntimeRemovalProcessor, RuntimeRemoveInstances
	 */
	void PopAllEntitiesPendingDestruction(TArray<FMassEntityHandle>& InOutEntities);
	
protected:
	FMassEntityHandle GetEntityHandleForIndex(const int32 Index) const
	{
		return Entities.IsValidIndex(Index) ? Entities[Index] : FMassEntityHandle();
	}

	//~ Begin UObject Overrides
	UE_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;
	UE_API virtual void PostLoad() override;
	UE_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	UE_API virtual void RegisterReplicationFragments(UE::Net::FFragmentRegistrationContext& Context, UE::Net::EFragmentRegistrationFlags RegistrationFlags) override;
	virtual bool IsNameStableForNetworking() const override { return true; }
	//~ End UObject Overrides

	// Called on BeginPlay to create default entity template
	UE_API void CreateEntityTemplate(const AActor& ExemplarActor);
	UE_API virtual void ModifyEntityTemplate(FMassEntityTemplateData& ModifiedTemplate, const AActor& ExemplarActor);

	// Called from Deinitialize to destroy the owned FMassEntityTemplate  
	UE_API void ReleaseEntityTemplate();

	// Helper function used in ApplyInstanceDeltas to apply a single delta
	// @see ApplyInstanceDeltas
	UE_API virtual void ApplyInstanceDelta(FMassEntityManager& EntityManager, const FInstancedActorsDelta& InstanceDelta, TArray<FInstancedActorsInstanceIndex>& OutEntitiesToRemove);

	// Helper function used in RollbackInstanceDeltas to rollback a single delta
	// @see RollbackInstanceDeltas
	UE_API virtual void RollbackInstanceDelta(FMassEntityManager& EntityManager, const FInstancedActorsDelta& InstanceDelta);

	// Unlink InstanceToUnlink's Actor (if any) from Mass by clearing the entities reference to it
	// and disconnecting from any subscribed actor signals, essentially 'forgetting' about the actor.
	//
	// Called by EjectInstanceActor to transfer the actor's management to persistence system and OnInstancedActorDestroyed
	// to allow the Actor to complete death process in the background after we've destroyed the Mass entity (which
	// would have otherwise force-destroyed the associated actor)
	//
	// @return The unlinked actor (if one was currently linked, nullptr otherwise)
	UE_API AActor* UnlinkActor(FInstancedActorsInstanceIndex InstanceToUnlink);

	// True if SetupLoadedInstances has ever been called
	bool bHasSetupLoadedInstances : 1 = false;
	
	// True if Initialize has ever been called
	bool bHasEverInitialized : 1 = false;

	// True during RuntimeRemoveInstances to prevent recursion via OnInstancedActorDestroyed
	bool bRemovingInstances : 1 = false;

	// Sets of ISMCs which instances can swap between to change 'visualization' e.g: 'with berries', 'without berries'
	//
	// InstanceVisualizations[0] is the default visualization, auto-created in BeginPlay from ActorClass and used
	// by default for all spawned entities in BeginPlay.
	//
	// Additional visualizations can be registered at runtime with AddVisualization, for instances to switch to (@todo)
	// @see AddVisualization, AddVisualizationAsync
	UPROPERTY(Transient, VisibleAnywhere, Category=InstancedActors)
	TArray<FInstancedActorsVisualizationInfo> InstanceVisualizations;

	// Allocation bit flags for InstanceVisualizations to allow for TSpareArray-like removal of visualization without affecting remaining
	// visualization indices. Size-matched to InstanceVisualizations with a bit per visualization defining whether or not that visualization
	// entry is actually allocated.
	// Note: TSparseArray can't be used directly here as it's not a UPROPERTY type.
	TBitArray<> InstanceVisualizationAllocationFlags;

	// Adds or reuses a previously removed entry in InstanceVisualizations
	// @return The 'visualization index` of the new or reused entry in InstanceVisualizations
	// @see AddVisualization, AddVisualizationAsync
	UE_API uint8 AllocateVisualization();

	// Creates ISMCs for VisualizationDesc.InstancedMeshes, registers them with Mass and sets
	// FInstancedActorsVisualizationInfo::MassStaticMeshDescIndex with the newly registed ISMC decription index
	// @see AddVisualization, AddVisualizationAsync
	UE_API void InitializeVisualization(uint8 AllocatedVisualizationIndex, const FInstancedActorsVisualizationDesc& VisualizationDesc);

#if WITH_EDITORONLY_DATA
	// ISMCs created in GetOrCreateActorInstanceData to match default visualizations ISMComponents for editor only preview of instances
	UPROPERTY()
	TArray<TObjectPtr<UInstancedStaticMeshComponent>> EditorPreviewISMComponents;
#endif

	UPROPERTY()
	FBox CachedLocalBounds = FBox(ForceInit);

private:
	void QueueEntitiesForDestruction(TArray<FMassEntityHandle>&& EntitiesToDestroy);

	mutable FCriticalSection PendingDestructionCS;

	// These will get destroyed next time commands are flushed.
	// @see QueueEntitiesForDestruction, PopAllEntitiesPendingDestruction
	TArray<FMassEntityHandle> EntitiesPendingDestruction;

	// Represents the shared fragment registered with MassEntityManager, that points back to this UInstancedActorsData instance
	TStructView<FInstancedActorsDataSharedFragment> SharedInstancedActorDataStruct;

	// List of FInstancedActorsDelta's to apply to instances, replicated via fast array replication to clients.
	// Formed at runtime via player actions and persisted / restored by game's persistence system
	// @see Serialize
	UPROPERTY(Replicated, SaveGame, Transient)
	FInstancedActorsDeltaList InstanceDeltas;

	UPROPERTY(Transient)
	FMassEntityConfig EntityConfig;

	// Result of combining AdditionalTags with class-based FInstancedActorsSettings::GameplayTags
	UPROPERTY(Transient)
	FGameplayTagContainer CombinedTags;

	bool bCanHydrate = true;

	TSharedPtr<UE::InstancedActors::FExemplarActorData> ExemplarActorData;

	struct FSetReplicatedActorRequests
	{
		FSetReplicatedActorRequests(FInstancedActorsInstanceIndex InInstanceIndex, TWeakObjectPtr<AActor> InActor)
			: InstanceIndex(InInstanceIndex), Actor(InActor)
		{	
		}

		FInstancedActorsInstanceIndex InstanceIndex;
		TWeakObjectPtr<AActor> Actor;
	};
	TArray<FSetReplicatedActorRequests> CachedSetReplicatedActorRequests;
};

//-----------------------------------------------------------------------------
// INLINES
//-----------------------------------------------------------------------------
inline bool UInstancedActorsData::CanHydrate() const
{
	return bCanHydrate;
}

#undef UE_API
