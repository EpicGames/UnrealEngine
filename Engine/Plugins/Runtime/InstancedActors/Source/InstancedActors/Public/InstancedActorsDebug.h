// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DrawDebugHelpers.h"
#include "EngineDefines.h"
#include "Logging/LogMacros.h"
#include "MassCommonTypes.h"
#include "Misc/Build.h"
#include "VisualLogger/VisualLogger.h"


#define WITH_INSTANCEDACTORS_DEBUG (!(UE_BUILD_SHIPPING || UE_BUILD_SHIPPING_WITH_EDITOR || UE_BUILD_TEST) && WITH_MASSGAMEPLAY_DEBUG && 1)

class AInstancedActorsManager;
class UInstancedActorsModifierVolumeComponent;
struct FInstancedActorsIterationContext;

#if WITH_INSTANCEDACTORS_DEBUG
namespace UE::InstancedActors::Debug
{
	namespace CVars
	{
		extern int32 DebugModifiers;
		extern int32 DebugManagerLoading;
		extern int32 DebugInstanceLoading;
	}

	// Draw mode constants for CVar comparison
	namespace DrawMode
	{
		constexpr int32 None = 0;
		constexpr int32 DebugDraw = 1;
		constexpr int32 VisLog = 2;
		constexpr int32 Both = 3;
	};

	inline bool ShouldDebugDraw(const int32& DebugDrawMode) { return DebugDrawMode == DrawMode::DebugDraw || DebugDrawMode >= DrawMode::Both; }
	inline bool ShouldVisLog(const int32& DebugDrawMode) { return DebugDrawMode == DrawMode::VisLog || DebugDrawMode >= DrawMode::Both; }

	void DebugDrawManager(const int32& DebugDrawMode, const AInstancedActorsManager& Manager);
	void LogInstanceCountsOnScreen(const int32& DebugDrawMode, const AInstancedActorsManager& Manager, float TimeToDisplay = 4.0f, FColor Color = FColor::Magenta);
	void DebugDrawAllInstanceLocations(const int32& DebugDrawMode, ELogVerbosity::Type Verbosity, const AInstancedActorsManager& Manager, const TOptional<FColor> Color, const UObject* LogOwner, const FName& CategoryName = TEXT("LogInstancedActors"));
	void DebugDrawModifierVolumeBounds(const int32& DebugDrawMode, const UInstancedActorsModifierVolumeComponent& ModifierVolume, const FColor& Color); 
	void DebugDrawModifierVolumeAddedToManager(const int32& DebugDrawMode, const AInstancedActorsManager& Manager, const UInstancedActorsModifierVolumeComponent& AddedModifierVolume);

	template <typename CategoryType, typename FmtType, typename... Types>
	void DebugDrawLocation(const int32& DebugDrawMode, const UWorld* World, const UObject* LogOwner, const CategoryType& CategoryName, ELogVerbosity::Type Verbosity, const FVector& Location, float Size, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		if (ShouldDebugDraw(DebugDrawMode))
		{
			#if ENABLE_DRAW_DEBUG
				::DrawDebugPoint(World, Location, Size, Color, /*bPersistent*/true);
			#endif // ENABLE_DRAW_DEBUG
		}
		if (ShouldVisLog(DebugDrawMode))
		{
			#if ENABLE_VISUAL_LOG
				// UE_VLOG_LOCATION but using LocationLogf directly to pass Verbosity by value
				if (FVisualLogger::IsRecording()) FVisualLogger::LocationLogf(LogOwner, CategoryName, Verbosity, Location, Size, Color, Fmt, Args...);
			#endif // ENABLE_VISUAL_LOG
		}
	}

	template <typename CategoryType, typename FmtType, typename... Types>
	void DebugDrawSphere(const int32& DebugDrawMode, const UWorld* World, const UObject* LogOwner, const CategoryType& CategoryName, ELogVerbosity::Type Verbosity, const FSphere& Sphere, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		if (ShouldDebugDraw(DebugDrawMode))
		{
			#if ENABLE_DRAW_DEBUG
				::DrawDebugSphere(World, Sphere.Center, Sphere.W, /*Segments*/10, Color, /*bPersistent*/true);
			#endif // ENABLE_DRAW_DEBUG
		}
		if (ShouldVisLog(DebugDrawMode))
		{
			#if ENABLE_VISUAL_LOG
				// UE_VLOG_LOCATION but using SphereLogf directly to pass Verbosity by value
				if (FVisualLogger::IsRecording()) FVisualLogger::SphereLogf(LogOwner, CategoryName, Verbosity, Sphere.Center, Sphere.W, Color, /*bWireframe = */false, Fmt, Args...);
			#endif // ENABLE_VISUAL_LOG
		}
	}

	template <typename CategoryType, typename FmtType, typename... Types>
	void DrawDebugSolidBox(const int32& DebugDrawMode, const UWorld* World, const UObject* LogOwner, const CategoryType& CategoryName, ELogVerbosity::Type Verbosity, const FBox& Box, const FColor& Color, const FmtType& Fmt, Types... Args)
	{
		if (ShouldDebugDraw(DebugDrawMode))
		{
			#if ENABLE_DRAW_DEBUG
				::DrawDebugSolidBox(World, Box, Color, FTransform::Identity, /*bPersistent*/true);
			#endif // ENABLE_DRAW_DEBUG
		}
		if (ShouldVisLog(DebugDrawMode))
		{
			#if ENABLE_VISUAL_LOG
				// UE_VLOG_BOX but using GeometryShapeLogf directly to pass Verbosity by value
				if (FVisualLogger::IsRecording()) FVisualLogger::BoxLogf(LogOwner, CategoryName, Verbosity, Box, FMatrix::Identity, Color, /*bWireframe = */false, Fmt, Args...);
			#endif // ENABLE_VISUAL_LOG
		}
	}
}
#endif // WITH_INSTANCEDACTORS_DEBUG
