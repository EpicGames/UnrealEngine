// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsDebug.h"
#if WITH_INSTANCEDACTORS_DEBUG
#include "MassEntityQuery.h"
#endif // WITH_INSTANCEDACTORS_DEBUG
#include "MassProcessor.h"
#include "InstancedActorsDebugProcessor.generated.h"


UCLASS()
class UInstancedActorsDebugProcessor : public UMassProcessor
{
	GENERATED_BODY()

protected:
	UInstancedActorsDebugProcessor();

	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;

	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

#if WITH_INSTANCEDACTORS_DEBUG
	/**
	 *  The query used when drawing instanced actors at Detailed batch LOD 
	 *  (see UE::InstancedActors::Debug::bDebugDrawDetailedCurrentRepresentation)
	 */
	FMassEntityQuery DetailedLODEntityQuery;
	
	/** The query used when "debug all entities" is enabled (see UE::InstancedActors::Debug::bDebugDrawAllEntities)*/
	FMassEntityQuery DebugAllEntityQuery;
#endif // WITH_INSTANCEDACTORS_DEBUG
};
