// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "Serialization/StructuredArchive.h"
#include "InstancedActorsIndex.generated.h"

#define UE_API INSTANCEDACTORS_API


class UInstancedActorsData;
class AInstancedActorsManager;
struct FInstancedActorsIterationContext;

/** This type is only valid to be used with the instance of UInstancedActorsData it applies to. */
USTRUCT()
struct FInstancedActorsInstanceIndex
{
	GENERATED_BODY()

	static constexpr uint16 INSTANCED_INDEX_NONE = MAX_uint16;
	static constexpr uint16 INSTANCED_INDEX_MAX = MAX_uint16 - 1;
	
	FInstancedActorsInstanceIndex() = default;
	explicit FInstancedActorsInstanceIndex(const int32 InIndex)
	{ 
		check(InIndex >= 0 && InIndex <= INSTANCED_INDEX_MAX);
		Index = InIndex;
	}

	friend FArchive& operator<<(FArchive& Ar, FInstancedActorsInstanceIndex& InstanceIndex)
	{
		Ar << InstanceIndex.Index;
		return Ar;
	}

	friend void operator<<(FStructuredArchive::FSlot Slot, FInstancedActorsInstanceIndex& InstanceIndex)
	{
		Slot << InstanceIndex.Index;
	}

	friend uint32 GetTypeHash(const FInstancedActorsInstanceIndex& InstanceIndex)
	{
		return uint32(InstanceIndex.Index);
	}

	bool IsValid() const { return Index != INSTANCED_INDEX_NONE; }

	/** Returns a string suitable for debug logging to identify this instance. */
	UE_API FString GetDebugName() const;

	bool operator==(const FInstancedActorsInstanceIndex&) const = default;

	int32 GetIndex() const { return Index; }

	static constexpr inline int32 BuildCompositeIndex(const uint16 InstanceDataID, const int32 InstanceIndex)
	{
		check(InstanceIndex >= 0 && InstanceIndex <= INSTANCED_INDEX_MAX);

		const uint32 HighBits = InstanceDataID;
		const uint32 LowBits = InstanceIndex;
		check(LowBits <= MAX_uint16);
		return static_cast<int32>((HighBits << InstanceIndexBits) | LowBits);
	}

	static constexpr inline int32 ExtractInstanceDataID(const int32 CompositeIndex)
	{
		return CompositeIndex >> InstanceIndexBits;
	}

	static constexpr inline int32 ExtractInternalInstanceIndex(const int32 CompositeIndex)
	{
		return (CompositeIndex & InstanceIndexMask);
	}

private:
	/** Stable(consistent between client and server) instance index into UInstancedActorsData */
	UPROPERTY()
	uint16 Index = INSTANCED_INDEX_NONE;

	static constexpr int32 InstanceIndexBits = 16;
	static constexpr int32 InstanceIndexMask = (1 << InstanceIndexBits) - 1;
};

template<>
struct TStructOpsTypeTraits<FInstancedActorsInstanceIndex> : public TStructOpsTypeTraitsBase2<FInstancedActorsInstanceIndex>
{
	enum
	{
		WithCopy = true,
		WithIdenticalViaEquality = true,
	};
};

USTRUCT(BlueprintType)
struct FInstancedActorsInstanceHandle
{
	GENERATED_BODY()

	FInstancedActorsInstanceHandle() = default;
	UE_API FInstancedActorsInstanceHandle(UInstancedActorsData& InInstancedActorData, FInstancedActorsInstanceIndex InIndex);

	UInstancedActorsData* GetInstanceActorData() const { return InstancedActorData.Get(); }
	UE_API UInstancedActorsData& GetInstanceActorDataChecked() const;

	UE_API AInstancedActorsManager* GetManager() const;
	UE_API AInstancedActorsManager& GetManagerChecked() const;

	UE_API bool IsValid() const;

	/** Returns a string suitable for debug logging to identify this instance. */
	UE_API FString GetDebugName() const;

	bool operator==(const FInstancedActorsInstanceHandle&) const = default;
	FInstancedActorsInstanceIndex GetInstanceIndex() const { return Index; }
	int32 GetIndex() const { return Index.GetIndex(); }

	void Reset()
	{
		InstancedActorData = nullptr;
		Index = FInstancedActorsInstanceIndex();
	}

private:

	friend INSTANCEDACTORS_API uint32 GetTypeHash(const FInstancedActorsInstanceHandle& InstanceHandle);
	friend UInstancedActorsData;
	friend AInstancedActorsManager;
	friend FInstancedActorsIterationContext;

	/** Specific UInstancedActorsData responsible for this instance.Can be used to get to it's owning Manager. */
	UPROPERTY()
	TWeakObjectPtr<UInstancedActorsData> InstancedActorData = nullptr;

	/** Stable(consistent between client and server) instance index into UInstancedActorsData. */
	UPROPERTY()
	FInstancedActorsInstanceIndex Index;
};

#undef UE_API
