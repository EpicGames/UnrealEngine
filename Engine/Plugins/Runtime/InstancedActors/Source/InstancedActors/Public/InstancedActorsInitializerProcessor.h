// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "InstancedActorsInitializerProcessor.generated.h"


class UInstancedActorsData;

USTRUCT()
struct FInstancedActorsMassSpawnData
{
	GENERATED_BODY()

	TWeakObjectPtr<UInstancedActorsData> InstanceData;
};

/** Initializes the fragments of all entities that fit the query specified in ConfigureQueries, which are all considered Instanced Actors. */
UCLASS()
class UInstancedActorsInitializerProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UInstancedActorsInitializerProcessor();

protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};
