// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsIndex.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "InstancedActorsIteration.generated.h"

#define UE_API INSTANCEDACTORS_API


class AInstancedActorsManager;
class UInstancedActorsData;
struct FInstancedActorsInstanceHandle;
struct FInstancedActorsInstanceIndex;

/**
 * Provides useful functionality while iterating instances like safe instance deletion.
 * @see AInstancedActorsManager::ForEachInstance
 */
USTRUCT()
struct FInstancedActorsIterationContext
{
	GENERATED_BODY()

	// Destructor to ensure no pending actions remain
	UE_API ~FInstancedActorsIterationContext();

	/**
	 * Safely marks InstanceHandle for destruction at the end of iteration, to ensure iteration
	 * order isn't affected.
	 * Note: These deletions will NOT be be persisted as if a player had performed them, rather the deletions will
	 *       make it as if the items were never present.
	 * Note: This is safe to call before entity spawning as source instance data will simply be invalidated,
	 *       preventing later entity spawning.
	 */
	UE_API void RemoveInstanceDeferred(const FInstancedActorsInstanceHandle& InstanceHandle);

	/**
	 * Safely marks all instances in InstanceData for destruction at the end of iteration, to ensure iteration
	 * order isn't affected.
	 * Note: These deletions will NOT be be persisted as if a player had performed them, rather the deletions will
	 *       make it as if the items were never present.
	 * Note: This is safe to call before entity spawning as source instance data will simply be invalidated,
	 *       preventing later entity spawning.
	 */
	UE_API void RemoveAllInstancesDeferred(UInstancedActorsData& InstanceData);

	/**
	 * Safely marks all instances in Manager for destruction at the end of iteration, to ensure iteration
	 * order isn't affected.
	 * Note: These deletions will NOT be be persisted as if a player had performed them, rather the deletions will
	 *       make it as if the items were never present.
	 * Note: This is safe to call before entity spawning as source instance data will simply be invalidated,
	 *       preventing later entity spawning.
	 */
	UE_API void RemoveAllInstancesDeferred(AInstancedActorsManager& Manager);

	/** Perform deferred instance removals **/
	UE_API void FlushDeferredActions();

private:

	TMap<TObjectPtr<UInstancedActorsData>, TArray<FInstancedActorsInstanceIndex>> InstancesToRemove;
	TArray<TObjectPtr<UInstancedActorsData>> RemoveAllInstancesIADs;
	TArray<TObjectPtr<AInstancedActorsManager>> RemoveAllInstancesIAMs;
};

/** Subclass of FInstancedActorsIterationContext that calls FlushDeferredActions in it's destructor */
struct FScopedInstancedActorsIterationContext : public FInstancedActorsIterationContext
{
	~FScopedInstancedActorsIterationContext()
	{
		FlushDeferredActions();
	}
};

#undef UE_API
