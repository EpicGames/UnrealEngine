// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ArrayView.h"
#include "InstancedActorsTypes.h"
#include "InstancedActorsIndex.h"
#include "MassEntityQuery.h"
#include "ActorPartition/PartitionActor.h"
#include "Containers/BitArray.h"
#include "Templates/SharedPointer.h"

#include "Engine/ActorInstanceManagerInterface.h"
#include "Elements/SMInstance/SMInstanceManager.h"

#include "InstancedActorsManager.generated.h"

#define UE_API INSTANCEDACTORS_API


class UInstancedActorsModifierVolumeComponent;
class UInstancedActorsSubsystem;
struct FInstancedActorsSettings;
struct FMassEntityManager;
class UInstancedActorsData;
class UInstancedStaticMeshComponent;

namespace UE::InstancedActors
{
	enum class EInsideBoundsTestResult
	{
		NotInside,
		OverlapLocation,
		OverlapBounds
	};
	
	enum class EBoundsTestType
	{
		Intersect,
		Enclosed,
		
		Default = Intersect
	};

	namespace CVars
	{
		extern INSTANCEDACTORS_API bool bEnablePersistence;
	}

	template <typename TBoundsType>
	bool PassesBoundsTest(const TBoundsType& QueryBounds, EBoundsTestType BoundsTestType, const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform);

} // UE::InstancedActors

DECLARE_STATS_GROUP(TEXT("InstanceActor Rendering"), STATGROUP_InstancedActorsRendering, STATCAT_Advanced);

// @todo consider renaming to AStaticInstancedActorsManager. Will require renaming of the instance data type as well along
//	with providing base classes to be used in generic code. 

/**
 * Regional manager of 'instanced actors'.
 *
 * Uses Mass to provide lightweight and efficient instancing of items in the distance, with server-authoritative actor
 * spawning around players. AInstancedActorsManager's also provide replication and persistence for their managed instances.
 *
 * Spawned and populated *offline* by UInstancedActorsSubsystem::InstanceActor. Offline population ensures client & server
 * both load the same stable instance data and can commonly refer to instances by index as such.
 */
UCLASS(MinimalAPI, Config = Mass)
class AInstancedActorsManager : public APartitionActor, public ISMInstanceManager, public IActorInstanceManagerInterface
{
	GENERATED_BODY()

public:
	UE_API AInstancedActorsManager();

	/** 
	 * Adds modifiers already registered with InInstancedActorSubsystem and either calls InitializeModifyAndSpawnEntities to spawn
	 * entities immediately, or schedules deferred call by InInstancedActorSubsystem if IA.DeferSpawnEntities is enabled.
	 * Called either in BeginPlay if InInstancedActorSubsystem was already initialized or latently once it is, in
	 * UInstancedActorsSubsystem::Initialize
	 */
	UE_API void OnAddedToSubsystem(UInstancedActorsSubsystem& InInstancedActorSubsystem, FInstancedActorsManagerHandle InManagerHandle);

	FInstancedActorsManagerHandle GetManagerHandle() const;

	/** 
	 * Performs setup after all Instances have been loaded. Canonically called from PostLoad(), but may need to be called manually
	 * if this AInstancedActorsManager is created at cook/runtime
	 */
	UE_API void SetupLoadedInstances();

	/** 
	 * Initializes all PerActorClassInstanceData, applies pre-spawn modifiers, spawns entities then applies post-spawn modifiers.
	 * 
	 * Called either directly in OnAddedToSubsystem or deferred and time-sliced in UInstancedActorsSubsystem::Tick if
	 * IA.DeferSpawnEntities is enabled.
	 */
	UE_API void InitializeModifyAndSpawnEntities();

	/** @return true if InstanceTransforms have been consumed to spawn Mass entities in InitializeModifyAndSpawnEntities */
	bool HasSpawnedEntities() const;

#if WITH_EDITOR
	/** Adds an instance of ActorClass at InstanceTransform location to instance data */
	UE_API FInstancedActorsInstanceHandle AddActorInstance(TSubclassOf<AActor> ActorClass, FTransform InstanceTransform, bool bWorldSpace = true, const FInstancedActorsTagSet& AdditionalInstanceTags = FInstancedActorsTagSet());

	/**
	 * Removes all instance data for InstanceHandle.
	 *
	 * This simply adds this instance to a FreeList which incurs extra cost to process at runtime before instance spawning.
	 *
	 * Note: By default UInstancedActorsSubsystem::RemoveActorInstance will destroy empty managers on
	 * last instance removal, implicitly clearing this for subsequent instance adds which would create
	 * a fresh manager.
	 *
	 * @see IA.CompactInstances console command
	 */
	UE_API bool RemoveActorInstance(const FInstancedActorsInstanceHandle& InstanceToRemove);
#endif

	/** Searches PerActorClassInstanceData, returning the IAD with matching UInstancedActorsData::ID, if any(nullptr otherwise) */
	UE_API UInstancedActorsData* FindInstanceDataByID(uint16 InstanceDataID) const;

	/** @return the full set of instance data for this manager */
	TConstArrayView<TObjectPtr<UInstancedActorsData>> GetAllInstanceData() const;

	/**
	 * Removes all instances as if they were never present i.e: these removals are not persisted as
	 * if made by a player.
	 * Prior to entity spawning this simply invalidates InstanceTransforms entries, post entity spawning this
	 * destroys spawned entities.
	 *
	 * Note: Any RuntimeRemoveInstances that have already been removed are safely skipped.
	 */
	UE_API void RuntimeRemoveAllInstances();

	/** @return the current valid instance count (i.e: NumInstances - FreeList.Num()) sum for all instance datas */
	UE_API int32 GetNumValidInstances() const;

	UE_API bool HasAnyValidInstances() const;

	/**
	 * @return true if InstanceHandle refers to this manager and we have current information for an
	 *	instance at InstanceHandle.InstanceIndex in InstanceHandle.InstancedActorData  
	 */
	UE_API bool IsValidInstance(const FInstancedActorsInstanceHandle& InstanceHandle) const;

	/** @return world space cumulative instance bounds. Only valid after BeginPlay. */
	FBox GetInstanceBounds() const;

	/**
	 * Iteration callback for ForEachInstance
	 * @param InstanceHandle	Handle to the current instance in the iteration
	 * @param InstanceTransform If entities have been spawned, this will be taken from the Mass transform fragment, else from UInstancedActorsData::InstanceTransforms
	 * @param InterationContext Provides useful functionality while iterating instances like safe instance deletion
	 * @return Return true to continue iteration to subsequent instances, false to break iteration.
	 */
	using FInstanceOperationFunc = TFunctionRef<bool(const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext)>;

	/**
	 * Predicate taking a UInstancedActorsData and returns true if IAD matches search criteria, false otherwise.
	 */
	using FInstancedActorDataPredicateFunc = TFunctionRef<bool(const UInstancedActorsData& InstancedActorData)>;

	/**
	 * Call InOperation for each valid instance in this manager. Prior to entity spawning in BeginPlay, this iterates valid UInstancedActorsData::InstanceTransforms.
	 * Once entities have been spawned, UInstancedActorsData::Entities are iterated.
	 * @param Operation Function to call for each instance found within QueryBounds
	 * @return false if InOperation ever returned false to break iteration, true otherwise.
	 */
	UE_API bool ForEachInstance(FInstanceOperationFunc Operation) const;
	UE_API bool ForEachInstance(FInstanceOperationFunc Operation, FInstancedActorsIterationContext& IterationContext, TOptional<FInstancedActorDataPredicateFunc> InstancedActorDataPredicate = TOptional<FInstancedActorDataPredicateFunc>()) const;

	/**
	 * Call InOperation for each valid instance in this manager whose location falls within QueryBounds. Prior to entity spawning in BeginPlay, this iterates valid
	 * UInstancedActorsData::InstanceTransforms. Once entities have been spawned, UInstancedActorsData::Entities are iterated.
	 * @param QueryBounds A world space FBox or FSphere to test instance locations against using QueryBounds.IsInside(InstanceLocation)
	 * @param InOperation Function to call for each instance found within QueryBounds
	 * @return false if InOperation ever returned false to break iteration, true otherwise.
	 */
	template <typename TBoundsType>
	bool ForEachInstance(const TBoundsType& QueryBounds, FInstanceOperationFunc InOperation) const;
	template <typename TBoundsType>
	bool ForEachInstance(const TBoundsType& QueryBounds, FInstanceOperationFunc InOperation, FInstancedActorsIterationContext& IterationContext
		, TOptional<FInstancedActorDataPredicateFunc> InstancedActorDataPredicate = TOptional<FInstancedActorDataPredicateFunc>()) const;

	/**
	 * Checks whether there are any instanced actors within this manager, representing ActorClass or its subclasses inside QueryBounds.
	 * The check doesn't differentiate between hydrated and dehydrated actors (i.e. whether there's an actor instance
	 * associated with the instance or not).
	 * @param bTestActorsIfSpawned if true then when an instance is found to overlap given bounds, and it has an actor 
	 *	spawned associated with it, then the actor itself will be tested against the bounds for more precise test.
	 */
	UE_API bool HasInstancesOfClass(const FBox& QueryBounds, TSubclassOf<AActor> ActorClass, const bool bTestActorsIfSpawned = false
		, const EInstancedActorsBulkLODMask AllowedLODs = EInstancedActorsBulkLODMask::All) const;

	/** 
	 * Determines whether the actor instance given by InstanceHandle overlaps QueryBounds. The test involves calculating 
	 * bounding box of the actor representation of the given instance (i.e. it's not only the transform that's being tested).
	 */
	template <typename TBoundsType>
	static UE::InstancedActors::EInsideBoundsTestResult IsInstanceInsideBounds(const TBoundsType & QueryBounds
		, const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform & InstanceTransform);

	/** Outputs instance metrics to Ar */
	UE_API void AuditInstances(FOutputDevice& Ar, bool bDebugDraw = false, float DebugDrawDuration = 10.0f) const;

	/** Called by IA.CompactInstances console command to fully remove FreeList instances */
	UE_API void CompactInstances(FOutputDevice& Ar);

	UE_API void AddModifierVolume(UInstancedActorsModifierVolumeComponent& ModifierVolume);
	UE_API void RemoveModifierVolume(UInstancedActorsModifierVolumeComponent& ModifierVolume);
	UE_API void RemoveAllModifierVolumes();

	/** Request the persistent data system to re-save this managers persistent data */
	UE_API void RequestPersistentDataSave();

	/** Helper function to deduce appropriate instanced static mesh bounds for ActorClass */
	static UE_API FBox CalculateBounds(TSubclassOf<AActor> ActorClass);

	/** @return the Mass entity manager used to spawn entities. Valid only after BeginPlay */
	TSharedPtr<FMassEntityManager> GetMassEntityManager() const;
	FMassEntityManager& GetMassEntityManagerChecked() const;

	/** @return the Instanced Actor Subsystem this Manager is registered with. Valid only after BeginPlay */
	UInstancedActorsSubsystem* GetInstancedActorSubsystem() const;
	UInstancedActorsSubsystem& GetInstancedActorSubsystemChecked() const;

	//~ Begin APartitionActor Overrides
#if WITH_EDITOR
	UE_API virtual uint32 GetDefaultGridSize(UWorld* InWorld) const override;
	UE_API virtual FGuid GetGridGuid() const override;
	UE_API void SetGridGuid(const FGuid& InGuid);
#endif
	//~ End APartitionActor Overrides

	static UE_API void UpdateInstanceStats(int32 InstanceCount, EInstancedActorsBulkLOD LODMode, bool Increment);

	/**
	 * Registers Components as related to InstanceData for IActorInstanceManagerInterface-related purposes.
	 */
	UE_API void RegisterInstanceDatasComponents(const UInstancedActorsData& InstanceData, TConstArrayView<TObjectPtr<UInstancedStaticMeshComponent>> Components);

	/**
	 * Unregisters Component from IActorInstanceManagerInterface-related tracking. Note that the function will assert
	 * whether the Component has been registered in the first place
	 */
	UE_API void UnregisterInstanceDatasComponent(UInstancedStaticMeshComponent& Component);

	UE_API virtual void CreateISMComponents(const FInstancedActorsVisualizationDesc& VisualizationDesc, FConstSharedStruct SharedSettings
		, TArray<TObjectPtr<UInstancedStaticMeshComponent>>& OutComponents, const bool bEditorPreviewISMCs = false);

	//~ Begin UObject Overrides
	UE_API virtual void Serialize(FStructuredArchive::FRecord Record) override;
	//~ End UObject Overrides

protected:
	// mz@todo a temp glue to be removed once InstancedActors get moved out of original project for good. It's virtual and I don't like it.
	virtual void RequestActorSave(AActor* Actor) {}

	virtual void OnInitializeModifyAndSpawnEntities() {}

	//~ Begin AActor Overrides
	UE_API virtual void OnReplicationStartedForIris(const FOnReplicationStartedParams&) override;
	UE_API virtual void BeginPlay() override;
	UE_API virtual void EndPlay(const EEndPlayReason::Type EndPlayReason) override;
	UE_API virtual bool IsHLODRelevant() const override;
	UE_API virtual void PostLoad() override;
#if WITH_EDITOR
	virtual bool IsUserManaged() const override { return true; }
	UE_API virtual void GetStreamingBounds(FBox& OutRuntimeBounds, FBox& OutEditorBounds) const override;
#endif
	//~ End AActor Overrides

	//~ Begin IActorInstanceManagerInterface Overrides
	UE_API virtual int32 ConvertCollisionIndexToInstanceIndex(int32 InIndex, const UPrimitiveComponent* RelevantComponent) const override;
	UE_API virtual AActor* FindActor(const FActorInstanceHandle& Handle) override;
	UE_API virtual AActor* FindOrCreateActor(const FActorInstanceHandle& Handle) override;
	UE_API virtual UClass* GetRepresentedClass(const int32 InstanceIndex) const override;
	UE_API virtual ULevel* GetLevelForInstance(const int32 InstanceIndex) const override;
	UE_API virtual FTransform GetTransform(const FActorInstanceHandle& Handle) const override;
	//~ End IActorInstanceManagerInterface Overrides

	/** 
	 * Called by Serialize for SaveGame archives to save / load IAD persistence data
	 * @param Record		The archive record to read / write IAD save data to
	 * @param InstanceData	The InstanceData to serialize from / to. May be nullptr when loading if Record's IAD has been removed since saving.
	 * 						In this case, we still need to read Record to seek the archive past this IAD record consistently.
	 * @param TimeDelta		Real time in seconds since serialization (0 when saving)
	 */
	UE_API void SerializeInstancePersistenceData(FStructuredArchive::FRecord Record, UInstancedActorsData* InstanceData, int64 TimeDelta) const;

	/** Despawns all entities spawned by individual UInstancedActorsData instances. */
	UE_API virtual void DespawnAllEntities();

	UE_DEPRECATED(5.8, "Using DespawnAllEntities with a parameter is deprecated.")
	UE_API void DespawnAllEntities(bool bIsSwitchingSubsystems);

	/** Attempts to run any 'pending' modifiers in ModifierVolumes where are appropriate to run given HasSpawnedEntities
	 * Called in BeginPlay prior to, and then again after SpawnEntities. Also called in AddModifierVolume.
	 * @see UInstancedActorsModifierBase::bRequiresSpawnedEntities
	 */
	UE_API void TryRunPendingModifiers();

	/** Called when persistent data has been applied / restored */
	UE_API void OnPersistentDataRestored();

	/** Calculate cumulative local space instance bounds for all PerActorClassInstanceData */
	UE_API FBox CalculateLocalInstanceBounds() const;

#if WITH_EDITOR
public:
	/** Helper function to create and initialize per-actor-class UInstancedActorsData's, optionally further partitioned by AdditionalInstanceTags */
	UE_API UInstancedActorsData& GetOrCreateActorInstanceData(TSubclassOf<AActor> ActorClass, const FInstancedActorsTagSet& AdditionalInstanceTags, bool bCreateEditorPreviewISMCs = true);

protected:
	UE_API virtual UInstancedActorsData* CreateNextInstanceActorData(TSubclassOf<AActor> ActorClass, const FInstancedActorsTagSet& AdditionalInstanceTags);

	/** Used to set the right properties on the editor ISMCs so we can do per-instance selection. */
	UE_API virtual void PreRegisterAllComponents() override;
#endif

	/** Configures given ISMComponent for editor-preview purposes */
	static UE_API void SetUpEditorPreviewISMComponent(TNotNull<UInstancedStaticMeshComponent*> ISMComponent);

	UPROPERTY(Transient)
	TObjectPtr<UInstancedActorsSubsystem> InstancedActorSubsystem;

	UPROPERTY(Transient)
	FInstancedActorsManagerHandle ManagerHandle;

	TSharedPtr<FMassEntityManager> MassEntityManager;

	/** Saved Actor Guid. Initialized from the actor name in constructor */ 
	UPROPERTY(VisibleAnywhere, Category=InstancedActors)
	FGuid SavedActorGuid = FGuid();

	/** True if SpawnEntities has been called to spawn entities. Reset in EndPlay */
	UPROPERTY(Transient)
	bool bHasSpawnedEntities = false;

	/** True if SetupLoadedInstances has ever been called */
	bool bHasSetupLoadedInstances : 1 = false;

	/** 
	 * Incremented in GetOrCreateActorInstanceData to provide IAD's with a stable, unique identifier
	 * within this IAM.
	 * @see UInstancedActorsData::ID
	 */
	UPROPERTY()
	uint16 NextInstanceDataID = 0;

	/** Per-actor-class instance data populated by AddActorInstance */
	UPROPERTY(Instanced, VisibleAnywhere, BlueprintReadOnly, Category=InstancedActors)
	TArray<TObjectPtr<UInstancedActorsData>> PerActorClassInstanceData;

	/** World space cumulative instance bounds, calculated in BeginPlay */
	UPROPERTY(Transient)
	FBox InstanceBounds = FBox(ForceInit);

	/** Modifier volumes added via AddModifierVolume */
	TArray<TWeakObjectPtr<UInstancedActorsModifierVolumeComponent>> ModifierVolumes;

	/** 
	 * A bit flag per volume in ModifierVolumes for whether the volume has pending Modifiers to run on this manager
	 * i.e if PendingModifierVolumeModifiers[VolumeIndex] has any true flags.
	 * Some modifiers can run prior to entity spawning for efficiency purposes, others must be held 'pending'
	 * later execution after entities have spawned.
	 */
	TBitArray<> PendingModifierVolumes;

	/** 
	 * A set of bit flags per volume in ModifierVolumes, matching each modifiers Modifiers list, marking
	 * whether the Modifier has yet to run on this manager or not (true = needs running).
	 * 
	 * Some modifiers can run prior to entity spawning for efficiency purposes, others must be held 'pending'
	 * later execution after entities have spawned.
	 */
	TArray<TBitArray<>> PendingModifierVolumeModifiers;

	UPROPERTY(Transient)
	TMap<TObjectPtr<UInstancedStaticMeshComponent>, int32> ISMComponentToInstanceDataMap;

	/** Class to be spawned to represent individual actor class instances. */
	UPROPERTY(EditAnywhere, Category=InstancedActor)
	TSubclassOf<UInstancedActorsData> InstancedActorsDataClass;

	mutable FMassEntityQuery InstancedActorLocationQuery;

private:
#if WITH_EDITORONLY_DATA
	UPROPERTY()
	FGuid ManagerGridGuid;

	/** Set this to false to be able to move the instances contained by this IAM.The property is not saved and will reset. */
	UPROPERTY(EditAnywhere, Transient, meta = (DisplayPriority = 1), Category = InstancedActors)
	bool bLockInstanceLocation = true;
#endif

	//~ Begin ISMInstanceManager Overrides
	UE_API virtual FText GetSMInstanceDisplayName(const FSMInstanceId& InstanceId) const override;
	UE_API virtual FText GetSMInstanceTooltip(const FSMInstanceId& InstanceId) const override;
	UE_API virtual bool CanEditSMInstance(const FSMInstanceId& InstanceId) const override;
	UE_API virtual bool CanMoveSMInstance(const FSMInstanceId& InstanceId, const ETypedElementWorldType WorldType) const override;
	UE_API virtual bool GetSMInstanceTransform(const FSMInstanceId& InstanceId, FTransform& OutInstanceTransform, bool bWorldSpace = false) const override;
	UE_API virtual bool SetSMInstanceTransform(const FSMInstanceId& InstanceId, const FTransform& InstanceTransform, bool bWorldSpace = false, bool bMarkRenderStateDirty = false, bool bTeleport = false) override;
	UE_API virtual void NotifySMInstanceMovementStarted(const FSMInstanceId& InstanceId) override;
	UE_API virtual void NotifySMInstanceMovementOngoing(const FSMInstanceId& InstanceId) override;
	UE_API virtual void NotifySMInstanceMovementEnded(const FSMInstanceId& InstanceId) override;
	UE_API virtual void NotifySMInstanceSelectionChanged(const FSMInstanceId& InstanceId, const bool bIsSelected) override;
	UE_API virtual bool DeleteSMInstances(TArrayView<const FSMInstanceId> InstanceIds) override;
	UE_API virtual bool DuplicateSMInstances(TArrayView<const FSMInstanceId> InstanceIds, TArray<FSMInstanceId>& OutNewInstanceIds) override;
	//~ End ISMInstanceManager Overrides

	/**
	 * Try to extract the actor from the provided handle or from associated Mass Entity.
	 * When unable to retrieve it returns nullptr but also the associated EntityView so caller could reuse it to create the actor.
	 */
	UE_API AActor* FindActorInternal(const FActorInstanceHandle& Handle, FMassEntityView& OutEntityView, bool bEnsureOnMissingInstanceDataOrMassEntity) const;

	UE_API AActor* GetActorForInstance(const UInstancedActorsData& InstanceData, const int32 InstancedActorIndex) const;

	UE_API FInstancedActorsInstanceHandle ActorInstanceHandleFromFSMInstanceId(const FSMInstanceId& InstanceId) const;
};


//-----------------------------------------------------------------------------
// inlines
//-----------------------------------------------------------------------------
inline FInstancedActorsManagerHandle AInstancedActorsManager::GetManagerHandle() const
{ 
	return ManagerHandle; 
}

inline FBox AInstancedActorsManager::GetInstanceBounds() const
{ 
	return InstanceBounds; 
}

inline bool AInstancedActorsManager::HasSpawnedEntities() const
{ 
	return bHasSpawnedEntities; 
}

inline TConstArrayView<TObjectPtr<UInstancedActorsData>> AInstancedActorsManager::GetAllInstanceData() const
{ 
	return PerActorClassInstanceData; 
}

inline TSharedPtr<FMassEntityManager> AInstancedActorsManager::GetMassEntityManager() const 
{ 
	return MassEntityManager; 
}

inline FMassEntityManager& AInstancedActorsManager::GetMassEntityManagerChecked() const
{
	check(MassEntityManager.IsValid());
	return *MassEntityManager;
}

inline UInstancedActorsSubsystem* AInstancedActorsManager::GetInstancedActorSubsystem() const 
{ 
	return InstancedActorSubsystem; 
}

inline UInstancedActorsSubsystem& AInstancedActorsManager::GetInstancedActorSubsystemChecked() const
{
	check(InstancedActorSubsystem);
	return *InstancedActorSubsystem;
}

#undef UE_API
