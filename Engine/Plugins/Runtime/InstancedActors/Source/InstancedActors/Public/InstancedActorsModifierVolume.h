// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "UObject/UObjectGlobals.h"
#include "InstancedActorsModifierVolume.generated.h"

class UInstancedActorsModifierVolumeComponent;

/**
 * A 3D volume with a list of Modifiers to execute against any Instanced Actor's found within the volume.
 * @see UInstancedActorsModifierVolumeComponent
 */
UCLASS(MinimalAPI)
class AInstancedActorsModifierVolume : public AActor
{
	GENERATED_BODY()

public:

	INSTANCEDACTORS_API AInstancedActorsModifierVolume(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	inline UInstancedActorsModifierVolumeComponent* GetModifierVolumeComponent() const
	{
		return ModifierVolumeComponent;
	}

protected:
	
	UPROPERTY(Category = InstancedActorsModifierVolume, VisibleAnywhere, BlueprintReadOnly, meta = (ExposeFunctionCategories = "Instanced Actor Modifier Volume", AllowPrivateAccess = "true"))
	TObjectPtr<UInstancedActorsModifierVolumeComponent> ModifierVolumeComponent;
};

/**
 * A 3D volume that performs filtered removal of Instanced Actor's found within the volume.
 * @see URemoveInstancesModifierVolumeComponent
 */
UCLASS(MinimalAPI)
class AInstancedActorsRemovalModifierVolume : public AInstancedActorsModifierVolume
{
	GENERATED_BODY()

public:

	INSTANCEDACTORS_API	AInstancedActorsRemovalModifierVolume(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
};
