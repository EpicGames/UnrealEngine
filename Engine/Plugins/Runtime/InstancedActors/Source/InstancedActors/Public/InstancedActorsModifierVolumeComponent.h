// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsTypes.h"
#include "Components/PrimitiveComponent.h"
#include "Containers/BitArray.h"
#include "InstancedActorsModifierVolumeComponent.generated.h"


class AInstancedActorsManager;
class UInstancedActorsModifierBase;
class UInstancedActorsSubsystem;
struct FInstancedActorsIterationContext;

UENUM(BlueprintType)
enum class EInstancedActorsVolumeShape : uint8
{
	Box,
	Sphere
};

/**
 * A 3D volume component with a list of Modifiers to execute against any Instanced Actor's found within the volume.
 * @see UInstancedActorsModifierBase
 */
UCLASS(MinimalAPI, ClassGroup="Instanced Actors", HideCategories=(Object, HLOD, Lighting, VirtualTexture, Collision, TextureStreaming, Mobile, Physics, Tags, AssetUserData, Activation, Cooking, Navigation, Input, Mobility), Meta=(BlueprintSpawnableComponent))
class UInstancedActorsModifierVolumeComponent : public UPrimitiveComponent
{
	GENERATED_BODY()

public:

	INSTANCEDACTORS_API UInstancedActorsModifierVolumeComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	// Volume bounds shape
	// Note: Replicated only in the initial bunch
	UPROPERTY(Replicated, EditAnywhere, Category="Instanced Actor Modifier Volume")
	EInstancedActorsVolumeShape Shape = EInstancedActorsVolumeShape::Box;

	// Local space volume half-extent (size along side = Extent * 2 * ComponentScale3D). Used if Shape == Box
	// Note: Replicated only in the initial bunch
	UPROPERTY(Replicated, EditAnywhere, Category="Instanced Actor Modifier Volume", meta=(EditCondition="Shape==EInstancedActorsVolumeShape::Box", EditConditionHides))
	FVector Extent = FVector(1.0f);

	// Local space volume radius (diameter = Radius * 2 * ComponentScale3D). Used if Shape == Sphere
	// Note: Replicated only in the initial bunch
	UPROPERTY(Replicated, EditAnywhere, Category="Instanced Actor Modifier Volume", meta=(EditCondition="Shape==EInstancedActorsVolumeShape::Sphere", EditConditionHides))
	float Radius = 1.0f;

	UPROPERTY(EditAnywhere, Category="Instanced Actor Modifier Volume", Instanced)
	TArray<TObjectPtr<UInstancedActorsModifierBase>> Modifiers;

	/** 
	 * If true, instances within the same outer level as this modifier volume will be skipped for modification.
	 * Useful when placed in injected or streaming levels, to modify the root level they're place in e.g: to clear 
	 * space for the injected level.
	 */
	UPROPERTY(EditAnywhere, Category="Instanced Actor Modifier Volume")
	bool bIgnoreOwnLevelsInstances = false;

	/**
	* If set, instances in IAMs which belong to these levels will be skipped (unaffected) by this modifier volume. 
	* Useful for putting a modifier volume in a sub-level to affect the outer level it's placed in, but  
	* skipping the sub-level content.
	*/
	UPROPERTY(Replicated, EditAnywhere, AdvancedDisplay, Category="Instanced Actor Modifier Volume")
	TArray<TSoftObjectPtr<UWorld>> LevelsToIgnore;

	UPROPERTY(EditAnywhere, Category="Rendering")
	FColor Color = FColor::White;

	UPROPERTY(EditAnywhere, Category="Rendering")
	bool bDrawOnlyIfSelected = false;

	UPROPERTY(EditAnywhere, Category="Rendering")
	float LineThickness = 5.0f;

	// Finds managers already registered with InInstancedActorSubsystem and adds this modifier to them.
	//
	// Called either in BeginPlay if InInstancedActorSubsystem was already initialized or latently once it is, in 
	// UInstancedActorsSubsystem::Initialize
	void OnAddedToSubsystem(UInstancedActorsSubsystem& InstancedActorSubsystem, FInstancedActorsModifierVolumeHandle InModifierVolumeHandle);

	// Called on addition to Manager in AInstancedActorsManager::AddModifierVolume
	void OnAddedToManager(AInstancedActorsManager& Manager);

	// Called on removal from Manager in AInstancedActorsManager::RemoveModifierVolume
	void OnRemovedFromManager(AInstancedActorsManager& Manager);

	/**
	 * Executes Modifiers for all instances in Manager overlapped by this volume.
	 * 
	 * If the entire Manager is enveloped by this volume, UInstancedActorsModifierBase::ModifyAllInstances 
	 * will be called, providing a fast path opportunity for whole-manager modification. By default, this simply
	 * calls UInstancedActorsModifierBase::ModifyInstance for all instances.
	 * 
	 * For partial overlaps, UInstancedActorsModifierBase::ModifyInstance will be called for only
	 * those instances in Manager whos instance locations are contained within this volume.
	 * 
	 * @param Manager				The Manager to execute Modifiers on.
	 * @param InOutPendingModifiers	Bit-flag array with flag per Modifiers whether the modifier is pending execution 
	 * 								for Manager. 
	 * 								If the modifier can run (@see UInstancedActorsModifierBase::bRequiresSpawnedEntities)
	 * 								the bit-flag will be set false after execution.
	 * @return true if all pending modifiers were run, false if any remain pending.
	 */
	bool TryRunPendingModifiers(AInstancedActorsManager& Manager, TBitArray<>& InOutPendingModifiers);

	//~ Begin UObject overrides
	INSTANCEDACTORS_API virtual void GetLifetimeReplicatedProps(TArray<FLifetimeProperty>& OutLifetimeProps) const override;
	//~ End UObject overrides

	//~ Begin UActorComponent overrides
	INSTANCEDACTORS_API virtual void BeginPlay() override;
	INSTANCEDACTORS_API virtual void EndPlay(EEndPlayReason::Type Reason) override;
	//~ End UActorComponent overrides

	//~ Begin USceneComponent overrides
	INSTANCEDACTORS_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;
	//~ End USceneComponent overrides

	//~ Begin USceneComponent overrides
	INSTANCEDACTORS_API FPrimitiveSceneProxy* CreateSceneProxy();
	//~ End USceneComponent overrides

protected:

	UPROPERTY(Transient)
	FInstancedActorsModifierVolumeHandle ModifierVolumeHandle;
		
	UPROPERTY(Transient)
	TArray<TWeakObjectPtr<AInstancedActorsManager>> ModifiedManagers;
};

/** A UInstancedActorsModifierVolumeComponent with a URemoveInstancedActorsModifier modifier pre-added to Modifiers */
UCLASS(MinimalAPI, Meta=(BlueprintSpawnableComponent))
class URemoveInstancesModifierVolumeComponent : public UInstancedActorsModifierVolumeComponent
{
	GENERATED_BODY()

	URemoveInstancesModifierVolumeComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
};
