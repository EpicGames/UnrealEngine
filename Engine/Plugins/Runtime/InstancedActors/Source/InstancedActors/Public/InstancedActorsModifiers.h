// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"
#include "InstancedActorsSettingsTypes.h"
#include "InstancedActorsSubsystem.h"
#include "InstancedActorsModifiers.generated.h"

#define UE_API INSTANCEDACTORS_API


struct FInstancedActorsInstanceHandle;
struct FInstancedActorsIterationContext;

/**
 * Base class for 'modifier' operations to run against Instanced Actors within AInstancedActorsManager's
 * 
 * Used by UInstancedActorsModifierVolumeComponent's to modify instances within their volumes. 
 * 
 * Subclasses must implement at least ModifyInstance but can also override ModifyAllInstances to provide a 
 * whole-manager fast path.
 *
 * @see UInstancedActorsModifierVolumeComponent
 */
UCLASS(MinimalAPI, Abstract)
class UInstancedActorsModifierBase : public UObject
{
	GENERATED_BODY()
public:

	/** Optional tag query to filter instances to modify */
	UPROPERTY(EditAnywhere, Category = InstancedActors)
	FGameplayTagQuery InstanceTagsQuery;

	/**
	 * If true, this modifier will wait to be called on Managers only after Mass
	 * entities have been spawned for all instances.
	 *
	 * @see bRequiresSpawnedEntities
	 */
	bool DoesRequireSpawnedEntities() const { return bRequiresSpawnedEntities; }

	/** 
	 * Callback to modify all instances in Manager, providing a 'fast path' opportunity for modifiers to perform whole-manager operations.
	 * By default this simply calls ModifyInstance for all instances.
	 * 
	 * @param Manager			The whole manager to modify. If bRequiresSpawnedEntities = false, this Manager may or may not have spawned 
	 * 							entities yet. @see bRequiresSpawnedEntities
	 * @param InterationContext Provides useful functionality while iterating instances like safe instance deletion
	 * 
	 * @see AInstancedActorsManager::ForEachInstance
	 */
	UE_API virtual void ModifyAllInstances(AInstancedActorsManager& Manager, FInstancedActorsIterationContext& IterationContext);

	/** 
	 * Per-instance callback to modify single instances.
	 * 
	 * Called by ModifyAllInstances & ModifyAllInstancesInBounds in their default implementations.
	 *
	 * @param InstanceHandle	Handle to the instance to Modify.
	 * @param InstanceTransform If entities have been spawned, this will be taken from the Mass transform fragment, else from 
	 * 							UInstancedActorsData::InstanceTransforms.
	 * @param InterationContext Provides useful functionality while iterating instances like safe instance deletion.
	 * 
	 * @return Return true to continue modification of subsequent instances, false to break iteration.
	 */
	virtual bool ModifyInstance(const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext) PURE_VIRTUAL(UInstancedActorsModifierBase::ModifyInstance, return false;)
	
	/** 
	 * Callback to modify all instances in Manager, whose location falls within Bounds. 
	 * Prior to entity spawning in BeginPlay, this iterates valid UInstancedActorsData::InstanceTransforms. Once entities have 
	 * spawned, UInstancedActorsData::Entities are iterated.
	 * 
	 * By default this simply calls ModifyInstance for all instances.
	 * 
	 * @param Bounds 			A world space FBox or FSphere to test instance locations against using Bounds.IsInside(InstanceLocation)
	 * @param Manager			The whole manager to modify. If bRequiresSpawnedEntities = false, this Manager may or may not have spawned entities yet. @see bRequiresSpawnedEntities
	 * @param InterationContext Provides useful functionality while iterating instances like safe instance deletion
	 * @see AInstancedActorsManager::ForEachInstance
	 */
	template<typename TBoundsType>
	void ModifyAllInstancesInBounds(const TBoundsType& Bounds, AInstancedActorsManager& Manager, FInstancedActorsIterationContext& IterationContext)
	{
		UE::InstancedActors::EBoundsTestType InstancedActorsDataBoundsTestType{UE::InstancedActors::EBoundsTestType::Default};
		
		Manager.ForEachInstance([this, &Bounds, &InstancedActorsDataBoundsTestType](const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext)
		{
			if (UE::InstancedActors::PassesBoundsTest(Bounds, InstancedActorsDataBoundsTestType, InstanceHandle, InstanceTransform))
			{
				return ModifyInstance(InstanceHandle, InstanceTransform, IterationContext);
			}
			return true;
		}, 
		IterationContext,
		/*Predicate*/TOptional<AInstancedActorsManager::FInstancedActorDataPredicateFunc>([this, &InstancedActorsDataBoundsTestType](const UInstancedActorsData& InstancedActorData)
		{			
			// Allow settings to stop modifiers affect this instance type.
			const FInstancedActorsSettings* Settings = InstancedActorData.GetSettingsPtr<const FInstancedActorsSettings>();
			if (Settings && Settings->bOverride_bIgnoreModifierVolumes && Settings->bIgnoreModifierVolumes)
			{
				return false;
			}

			if (InstanceTagsQuery.IsEmpty() || InstancedActorData.GetCombinedTags().MatchesQuery(InstanceTagsQuery))
			{
				// modify the EBoundsTestType for this set of Entities that are associated with this IAD, if the settings require it
				InstancedActorsDataBoundsTestType = (Settings != nullptr && Settings->bModifierVolumeCheckFullyEnclosed)
					? UE::InstancedActors::EBoundsTestType::Enclosed
					: UE::InstancedActors::EBoundsTestType::Intersect;

				return true;
			}

			return false;
		}));
	}

protected:

	/**
	 * If true, this modifier will wait to be called on Managers only after Mass
	 * entities have been spawned for all instances.
	 * 
	 * If false, this modifier may be called on managers prior to entity spawning (i.e: Manager.HasSpawnedEntities() = false)
	 * where operations like RuntimeRemoveEntities are cheaper to run pre-entity spawning. However, as latently spawned UInstancedActorsModifierVolumeComponent's
	 * may be matched up with Managers that have already spawned entities, modifiers with bRequiresSpawnedEntities = false must also
	 * support execution on Managers with HasSpawnedEntities() = true and should check Managers state accordingly in Modify
	 * callbacks. Modification is tracked per manager though and is guaranteed to only run once per manager, so modifiers that are
	 * run pre-entity spawning will not try and run again post-spawn.
	 * 
	 * @see AInstancedActorsManager::TryRunPendingModifiers
	 */
	bool bRequiresSpawnedEntities = true;
};

/** 
 * Modifier which removes all affected instances using AInstancedActorsManager::RuntimeRemoveInstances for individual instances.
 * For whole-manager modification this simply destroys the Manager.
 */
UCLASS(MinimalAPI)
class URemoveInstancedActorsModifier : public UInstancedActorsModifierBase
{
	GENERATED_BODY()
public:
	URemoveInstancedActorsModifier();

	virtual void ModifyAllInstances(AInstancedActorsManager& Manager, FInstancedActorsIterationContext& IterationContext) override;
	virtual bool ModifyInstance(const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext) override;
};

#undef UE_API
