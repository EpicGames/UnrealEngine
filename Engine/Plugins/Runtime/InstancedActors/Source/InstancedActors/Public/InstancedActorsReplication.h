// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsIndex.h"
#include "Net/Serialization/FastArraySerializer.h"
#include "InstancedActorsReplication.generated.h"

#define UE_API INSTANCEDACTORS_API


struct FInstancedActorsDeltaList;
class UInstancedActorsData;

/** Per-instance delta's against the cooked instance data, for persistence and replication */
USTRUCT() 
struct FInstancedActorsDelta : public FFastArraySerializerItem
{
	GENERATED_BODY()

	FInstancedActorsDelta(FInstancedActorsInstanceIndex InInstanceIndex)
		: InstanceIndex(InInstanceIndex)
	{}

	FInstancedActorsDelta() = default;

	/**
	 * Returns true if this delta actually contains any non-default 
	 * deltas / overrides to apply.
     */
	inline bool HasAnyDeltas() const
	{
		return IsDestroyed() 
			|| HasCurrentLifecyclePhase()
#if WITH_SERVER_CODE
			|| HasCurrentLifecyclePhaseTimeElapsed()
#endif
			;
	}

	FInstancedActorsInstanceIndex GetInstanceIndex() const { return InstanceIndex;  }

	bool IsDestroyed() const { return bDestroyed; }

	const bool HasCurrentLifecyclePhase() const { return CurrentLifecyclePhaseIndex != (uint8)INDEX_NONE; }
	uint8 GetCurrentLifecyclePhaseIndex() const { return CurrentLifecyclePhaseIndex; }

#if WITH_SERVER_CODE

	// mz@todo IA: move this section back
	const bool HasCurrentLifecyclePhaseTimeElapsed() const { return CurrentLifecyclePhaseTimeElapsed > 0.0f; }
	FFloat16 GetCurrentLifecyclePhaseTimeElapsed() const { return CurrentLifecyclePhaseTimeElapsed; }

#endif // WITH_SERVER_CODE

private:

	friend FInstancedActorsDeltaList;

	bool SetDestroyed(bool bInDestroyed) { return bDestroyed = bInDestroyed; }
	void SetCurrentLifecyclePhaseIndex(uint8 InCurrentLifecyclePhaseIndex) { CurrentLifecyclePhaseIndex = InCurrentLifecyclePhaseIndex; }
	void ResetLifecyclePhaseIndex() { CurrentLifecyclePhaseIndex = (uint8)INDEX_NONE; }

	UPROPERTY()
	FInstancedActorsInstanceIndex InstanceIndex;

	UPROPERTY()
	uint8 bDestroyed : 1 = false;

	UPROPERTY()
	uint8 CurrentLifecyclePhaseIndex = (uint8)INDEX_NONE;

#if WITH_SERVER_CODE
	void SetCurrentLifecyclePhaseTimeElapsed(FFloat16 InCurrentLifecyclePhaseTimeElapsed) { CurrentLifecyclePhaseTimeElapsed = InCurrentLifecyclePhaseTimeElapsed; }
	void ResetLifecyclePhaseTimeElapsed() { CurrentLifecyclePhaseTimeElapsed = -1.0f; }

	// Server-only (not replicated) time elapsed in current phase, saved & restored via persistence.
	// Unrequired by client code which only needs to know about discrete phase changes for visual updates.
	FFloat16 CurrentLifecyclePhaseTimeElapsed = -1.0f;
#endif // WITH_SERVER_CODE
};

USTRUCT()
struct FInstancedActorsDeltaList : public FFastArraySerializer
{
	GENERATED_BODY()

	UE_API void Initialize(UInstancedActorsData& InOwnerInstancedActorData);

	const TArray<FInstancedActorsDelta>& GetInstanceDeltas() const { return InstanceDeltas; }

	// Clear the InstanceDeltas list and resets InstancedActorData 
	// @param bMarkDirty If true, marks InstanceDeltas dirty for fast array replication
	UE_API void Reset(bool bMarkDirty = true);

	// Adds or modifies a FInstancedActorsDelta for InstanceIndex, marking the instance as destroyed and marks the 
	// delta as dirty for replication and application on clients.
	// This delta will also be persisted @see AInstancedActorsManager::SerializeInstancePersistenceData
	// Note: This does not request a persistence re-save
	UE_API void SetInstanceDestroyed(FInstancedActorsInstanceIndex InstanceIndex);

	UE_API void RemoveDestroyedInstanceDelta(FInstancedActorsInstanceIndex InstanceIndex);

	// Adds or modifies a FInstancedActorsDelta for InstanceIndex, specifying a new lifecycle phase to switch the instance
	// to, and marks the delta as dirty for replication and application on clients
	// Note: This does not request a persistence re-save
	UE_API void SetCurrentLifecyclePhaseIndex(FInstancedActorsInstanceIndex InstanceIndex, uint8 InCurrentLifecyclePhaseIndex);

	UE_API void RemoveLifecyclePhaseDelta(FInstancedActorsInstanceIndex InstanceIndex);

#if WITH_SERVER_CODE
	// Adds or modifies a FInstancedActorsDelta for InstanceIndex, specifying a new elapse time for the current lifecycle phase.
	// Note: This is a server-only delta and is NOT replicated to clients. It's simply stored in the delta list alongside the lifecycle 
	//		 phase index so they can be restored together from persistence and applied on the server in OnPersistentDataRestored -> ApplyInstanceDeltas
	UE_API void SetCurrentLifecyclePhaseTimeElapsed(FInstancedActorsInstanceIndex InstanceIndex, FFloat16 InCurrentLifecyclePhaseTimeElapsed);

	UE_API void RemoveLifecyclePhaseTimeElapsedDelta(FInstancedActorsInstanceIndex InstanceIndex);
#endif // WITH_SERVER_CODE

	const uint16 GetNumDestroyedInstanceDeltas() const { return NumDestroyedInstanceDeltas; }
	const uint16 GetNumLifecyclePhaseDeltas() const { return NumLifecyclePhaseDeltas; }
	const uint16 GetNumLifecyclePhaseTimeElapsedDeltas() const { return NumLifecyclePhaseTimeElapsedDeltas; }

	// UStruct overrides
	UE_API bool NetDeltaSerialize(FNetDeltaSerializeInfo& DeltaParams);

	// FFastArraySerializer overrides
	UE_API void PostReplicatedAdd(const TArrayView<int32>& AddedIndices, int32 FinalSize);
	UE_API void PostReplicatedChange(const TArrayView<int32>& ChangedIndices, int32 FinalSize);
	UE_API void PreReplicatedRemove(const TArrayView<int32>& RemovedIndices, int32 FinalSize);

private:

	UE_API FInstancedActorsDelta& FindOrAddInstanceDelta(FInstancedActorsInstanceIndex InstanceIndex);
	UE_API void RemoveInstanceDelta(uint16 DeltaIndex);

	// Lookup the InstanceDeltas index from a FInstancedActorsInstanceIndex. 
	// Note: This is server only data, initialized in Initialize
	TMap<FInstancedActorsInstanceIndex, uint16> InstanceIndexToDeltaIndex;

	// Cached counts for persistence serialization
	uint16 NumDestroyedInstanceDeltas = 0;
	uint16 NumLifecyclePhaseDeltas = 0;
	uint16 NumLifecyclePhaseTimeElapsedDeltas = 0;

	UPROPERTY(Transient)
	TArray<FInstancedActorsDelta> InstanceDeltas; // FastArray of Instance replication data.

	// Raw pointer to the UInstancedActorsData this FInstancedActorsDeltaList instance is a member of
	UInstancedActorsData* InstancedActorData = nullptr;
};

template<>
struct TStructOpsTypeTraits< FInstancedActorsDeltaList > : public TStructOpsTypeTraitsBase2< FInstancedActorsDeltaList >
{
	enum
	{
		WithNetDeltaSerializer = true,
	};
};

#undef UE_API
