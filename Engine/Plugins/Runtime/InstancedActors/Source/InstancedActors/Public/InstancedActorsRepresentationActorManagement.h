// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRepresentationActorManagement.h"
#include "InstancedActorsRepresentationActorManagement.generated.h"


enum class EUpdateTransformFlags : int32;
enum class ETeleportType : uint8;
struct FMassEntityView;

UCLASS(MinimalAPI)
class UInstancedActorsRepresentationActorManagement : public UMassRepresentationActorManagement
{
	GENERATED_BODY()

public:
	AActor* FindOrInstantlySpawnActor(UMassRepresentationSubsystem& RepresentationSubsystem, FMassEntityManager& EntityManager, FMassEntityView& EntityView);

	// Called by UInstancedActorsData::UnlinkActor to cleanup actor delegate callbacks
	INSTANCEDACTORS_API virtual void OnActorUnlinked(AActor& Actor);

protected:
	//~ Begin UMassRepresentationActorManagement Overrides
	INSTANCEDACTORS_API virtual EMassActorSpawnRequestAction OnPostActorSpawn(const FMassActorSpawnRequestHandle& SpawnRequestHandle
		, FConstStructView SpawnRequest, TSharedRef<FMassEntityManager> EntityManager) const override;

	// @todo make this behavior configurable
	/** Overriding to make sure ticking doesn't get enabled on spawned actors - we don't want that for handled actors */
	INSTANCEDACTORS_API virtual void SetActorEnabled(const EMassActorEnabledType EnabledType, AActor& Actor, const int32 EntityIdx
		, FMassCommandBuffer& CommandBuffer) const override;

	/** Overridden to skip transform updates for replicated actors on clients */
	INSTANCEDACTORS_API virtual void TeleportActor(const FTransform& Transform, AActor& Actor, FMassCommandBuffer& CommandBuffer) const;
	//~ End UMassRepresentationActorManagement Overrides

	INSTANCEDACTORS_API void OnSpawnedActorDestroyed(AActor& DestroyedActor, FMassEntityHandle EntityHandle) const;
	INSTANCEDACTORS_API void OnSpawnedActorMoved(USceneComponent* MovedActorRootComponent, EUpdateTransformFlags TransformUpdateFlags, ETeleportType TeleportType, FMassEntityHandle EntityHandle) const;
};
