// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRepresentationSubsystem.h"
#include "InstancedActorsRepresentationSubsystem.generated.h"


UCLASS()
class UInstancedActorsRepresentationSubsystem : public UMassRepresentationSubsystem
{
	GENERATED_BODY()

protected:
	//~ Begin USubsystem Overrides
	virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	virtual void Deinitialize() override;
	//~ End USubsystem Overrides

	void OnSettingsChanged();

	FDelegateHandle OnSettingsChangedHandle;
};
