// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"

#include "InstancedActorsRuntimeRemovalProcessor.generated.h"

/** 
 * This processor's sole responsibility is to find all UInstancedActorsData containing entities pending destruction
 * and to actually schedule their destruction. It also marks their FMassRepresentationFragment as bIsPendingDestruction
 * in order to prevent downstream processors from running any unnecessary representation logic on them.
 * @see UInstancedActorsData::RuntimeRemoveInstances, UInstancedActorsSubsystem::MarkIadDirtyWithEntitiesPendingDestruction
 */
UCLASS(MinimalAPI)
class UInstancedActorsRuntimeRemovalProcessor : public UMassProcessor
{
	GENERATED_BODY()
public:
	UInstancedActorsRuntimeRemovalProcessor();

protected:
	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const override { return false; }
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	/** Not directly used, but kept for the dependency solver */
	FMassEntityQuery EntityQuery;
};