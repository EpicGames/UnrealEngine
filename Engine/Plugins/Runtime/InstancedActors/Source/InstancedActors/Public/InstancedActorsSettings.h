// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataRegistryId.h"
#include "Engine/DeveloperSettings.h"
#include "UObject/SoftObjectPtr.h"
#include "MassActorSpawnerSubsystem.h"
#include "InstancedActorsSubsystem.h"
#include "MassStationaryDistanceVisualizationTrait.h"
#include "InstancedActorsSettings.generated.h"


#define GET_INSTANCEDACTORS_CONFIG_VALUE(a) (GetMutableDefault<UInstancedActorsProjectSettings>()->a)

USTRUCT()
struct FInstancedActorsConfig
{
	GENERATED_BODY()

	UPROPERTY(Config, EditAnywhere, Category = Subsystems, NoClear, meta = (MetaClass = "/Script/MassActors.MassActorSpawnerSubsystem"))
	TSubclassOf<UMassActorSpawnerSubsystem> ServerActorSpawnerSubsystemClass; 

	UPROPERTY(Config, EditAnywhere, Category = Subsystems, NoClear, meta = (MetaClass = "/Script/MassActors.MassActorSpawnerSubsystem"))
	TSubclassOf<UMassActorSpawnerSubsystem> ClientActorSpawnerSubsystemClass;

	UPROPERTY(Config, EditAnywhere, Category = Subsystems, NoClear, meta = (MetaClass = "/Script/InstancedActors.InstancedActorsSubsystem"))
	TSubclassOf<UInstancedActorsSubsystem> InstancedActorsSubsystemClass;

	UPROPERTY(Config, EditAnywhere, Category = Subsystems, NoClear, meta = (MetaClass = "/Script/MassRepresentation.MassStationaryDistanceVisualizationTrait"))
	TSubclassOf<UMassStationaryDistanceVisualizationTrait> StationaryVisualizationTraitClass;
};

USTRUCT()
struct FClassConfigOverrideEntry
{
	GENERATED_BODY()

	FObjectKey Owner;
	
	UPROPERTY()
	FInstancedActorsConfig ConfigOverride;
};

/** 
 * Configurable project settings for the Instanced Actors system.
 * @see FInstancedActorsClassSettingsBase and FInstancedActorsClassSettings for per-class specific runtime settings.
 * @see AInstancedActorsManager
 */
UCLASS(Config=InstancedActors, defaultconfig, DisplayName = "Instanced Actors", MinimalAPI)
class UInstancedActorsProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	DECLARE_MULTICAST_DELEGATE(FOnSettingsChanged);

	UInstancedActorsProjectSettings();

	INSTANCEDACTORS_API TSubclassOf<UMassActorSpawnerSubsystem> GetServerActorSpawnerSubsystemClass() const;
	INSTANCEDACTORS_API TSubclassOf<UMassActorSpawnerSubsystem> GetClientActorSpawnerSubsystemClass() const;
	INSTANCEDACTORS_API TSubclassOf<UInstancedActorsSubsystem> GetInstancedActorsSubsystemClass() const;
	INSTANCEDACTORS_API TSubclassOf<UMassStationaryDistanceVisualizationTrait> GetStationaryVisualizationTraitClass() const;

	void RegisterConfigOverride(UObject& Owner, const FInstancedActorsConfig& Config);
	void UnregisterConfigOverride(UObject& Owner);

	FOnSettingsChanged& GetOnSettingsUpdated() { return OnSettingsUpdated; }

protected:
	virtual void PostInitProperties() override;
#if WITH_EDITOR
	virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	void CompileSettings();

public:
	/** 3D grid size (distance along side) for partitioned instanced actor managers */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, meta = (ClampMin="0", Units=cm), Category = Grid)
	int32 GridSize = 24480;

	/** Data Registry to gather 'named' FInstancedActorsSettings from during UInstancedActorsSubsystem init */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = ActorClassSettings)
	FDataRegistryType NamedSettingsRegistryType = "InstancedActorsNamedSettings";

	/** Data Registry to gather per-class FInstancedActorsClassSettingsBase-based settings from during UInstancedActorsSubsystem init */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = ActorClassSettings)
	FDataRegistryType ActorClassSettingsRegistryType = "InstancedActorsClassSettings";

	/**
	 * If specified, these named settings will be applied to the default settings used as the base settings set for all 
	 * others, with a lower precedence than any per-class overrides 
	 */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = ActorClassSettings)
	FName DefaultBaseSettingsName = NAME_None;

	/** If specified, these named settings will be applied as a final set of overrides to all settings, overriding / taking precedence over all previous values */
	UPROPERTY(Config, EditAnywhere, BlueprintReadOnly, Category = ActorClassSettings)
	FName EnforcedSettingsName = NAME_None;

	UPROPERTY(Config, EditAnywhere, Category = ActorClassSettings)
	FInstancedActorsConfig DefaultConfig;

protected:
	FOnSettingsChanged OnSettingsUpdated;

	/** Represents the current config combining DefaultConfig and all registered ClassConfigOverrides */
	UPROPERTY(Transient)
	FInstancedActorsConfig CompiledActiveConfig;
		
	UPROPERTY(Transient)
	TArray<FClassConfigOverrideEntry> ClassConfigOverrides;
};
