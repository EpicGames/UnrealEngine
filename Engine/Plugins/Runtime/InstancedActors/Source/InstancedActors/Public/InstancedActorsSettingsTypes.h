// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DataTable.h"
#include "GameFramework/Actor.h"
#include "StructUtils/StructView.h"
#include "StructUtils/InstancedStruct.h"
#include "GameplayTagContainer.h"
#include "InstancedActorsSettingsTypes.generated.h"

#define UE_API INSTANCEDACTORS_API


// Helper macro for FInstancedActorsSettings::OverrideIfDefault
// 
// e.g: IASETTINGS_OVERRIDE_IF_DEFAULT(bInstancesCastShadows) produces:
//  
// if (OverrideSettings.bOverride_bInstancesCastShadows && !bOverride_bInstancesCastShadows)
// {
// 		bOverride_bInstancesCastShadows = true;
//		bInstancesCastShadows = OverrideSettings.bInstancesCastShadows;
// }
#define IASETTINGS_OVERRIDE_IF_DEFAULT(SettingName) if (OverrideSettings.bOverride_##SettingName && !bOverride_##SettingName) \
{ \
	bOverride_##SettingName = true; \
	SettingName = OverrideSettings.SettingName; \
}

// Helper macro for FInstancedActorsSettings::ToString
//
// e.g: IASETTINGS_OVERRIDE_TO_STRING(bInstancesCastShadows) produces:
//
// if (!bOverridesOnly || bOverride_bInstancesCastShadows)
// {
// 		SettingsString << TEXT("bInstancesCastShadows: ") << bInstancesCastShadows << TEXT(" ");
// }
#define IASETTINGS_SETTING_TO_STRING(SettingName) if (!bOverridesOnly || bOverride_##SettingName) \
{ \
	SettingsString << TEXT(#SettingName ": ") << SettingName << TEXT(" "); \
}

#define IASETTINGS_FLOAT_SETTING_TO_STRING(SettingName) if (!bOverridesOnly || bOverride_##SettingName) \
{ \
	SettingsString << TEXT(#SettingName ": ") << FString::SanitizeFloat(SettingName) << TEXT(" "); \
}

#define IASETTINGS_UOBJECT_SETTING_TO_STRING(SettingName) if (!bOverridesOnly || bOverride_##SettingName) \
{ \
	SettingsString << TEXT(#SettingName ": ") << SettingName->GetPathName() << TEXT(" "); \
}

/** 
 * Settings for controlling Instanced Actor behavior. Applied by 'actor class' either in groups via 'named settings' 
 * e.g: "SmallThings", "Trees" or on a specific class basis.
 * 
 * Named settings are FInstancedActorsSettings registered by name in UInstancedActorsProjectSettings::NamedSettingsRegistryType 
 * data registry.
 *
 * Class-specific settings are FInstancedActorsSettings registered by actor class via FInstancedActorsClassSettingsBase's in the 
 * UInstancedActorsProjectSettings::ActorClassSettingsRegistryType data registry. Named settings can be used as bases here via
 * FInstancedActorsClassSettingsBase::BaseSettings.
 *
 * @see UInstancedActorsProjectSettings for settings registry info.
 */
USTRUCT(BlueprintType)
struct FInstancedActorsSettings : public FTableRowBase
{
	GENERATED_BODY()

	// Bitflag per setting to choose if it should override a base setting

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_bInstancesCastShadows : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_MaxActorDistance : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_bDisableAutoDistanceCulling : 1 = false;

	//UE_DEPRECATED(5.4, "Layers Object no longer available.")
	UPROPERTY(meta = (DeprecatedProperty), Transient)
	uint8 bOverride_MaxInstanceDistance_DEPRECATED : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_MaxInstanceDistances : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_LODDistanceScales : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_AffectDistanceFieldLighting : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_DetailedRepresentationLODDistance : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_ForceLowRepresentationLODDistance : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_WorldPositionOffsetDisableDistance : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_bEjectOnActorMoved : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_ActorEjectionMovementThreshold : 1 = false;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_bCanEverAffectNavigation : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_OverrideWorldPartitionGrid : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_ScaleEntityCount : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_ActorClass : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_bCanBeDamaged : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_bIgnoreModifierVolumes : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_bModifierVolumeCheckFullyEnclosed : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_bControlPhysicsState : 1 = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta=(InlineEditConditionToggle), Category=InstancedActors)
	uint8 bOverride_GameplayTags : 1 = false;

	// Settings 

	/** Optional shadow casting override applied to instance ISMC's if set (shadow casting settings from ActorClass will be used for ISMC's if unset) */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_bInstancesCastShadows"), Category=InstancedActors)
	bool bInstancesCastShadows = true;

	/** Distance in cm from player to spawn actors within. Beyond this, Actors will be switched to ISMC instances up to MaxDrawDistance */
	UPROPERTY(EditAnywhere, Meta = (EditCondition = "bOverride_MaxActorDistance"), Category=InstancedActors)
	double MaxActorDistance = 1000.0;

	/** Disable auto computed distance culling using the bounding box of the static mesh - instance will render upto MaxInstanceDistance */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_bDisableAutoDistanceCulling"), Category=InstancedActors)
	bool bDisableAutoDistanceCulling = false;

	/** If false, collision will not be managed by the mass LODs for this instance **/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_bControlPhysicsState"), Category=InstancedActors)
	bool bControlPhysicsState = true;	

	static constexpr double DefaultMaxInstanceDistance = 100000.0;

	/** Final draw distance for ISMC instances */
	UPROPERTY(meta = (DeprecatedProperty), Transient)
	double MaxInstanceDistance_DEPRECATED = 100000.0;

	/** Scale applied on MaxDrawDistance for low quality level, see IA.ViewDistanceQuality */
	UPROPERTY(EditAnywhere, EditFixedSize, Meta = (EditFixedOrder, EditCondition = "bOverride_MaxInstanceDistances"), Category=InstancedActors)
	TArray<double> MaxInstanceDistances = { DefaultMaxInstanceDistance, DefaultMaxInstanceDistance, DefaultMaxInstanceDistance, DefaultMaxInstanceDistance };

	/** Sets LOD Distance Scale for low quality level, see IA.ViewDistanceQuality */
	UPROPERTY(EditAnywhere, EditFixedSize, Meta = (EditFixedOrder, EditCondition = "bOverride_LODDistanceScales"), Category=InstancedActors)
	TArray<float> LODDistanceScales = { 1.0f, 1.0f, 1.0f, 1.0f };

	/** If enabled, then asset will be used in distance field lighting */
	UPROPERTY(EditAnywhere, EditFixedSize, Meta = (EditFixedOrder, EditCondition = "bOverride_AffectDistanceFieldLighting"), Category=InstancedActors)
	TArray<bool> AffectDistanceFieldLighting = { true, true, true, true };

	/** Perform per-entity 'detailed' representation LOD calculation up to this distance from a viewer */
	UPROPERTY(EditAnywhere, Meta = (EditCondition = "bOverride_DetailedRepresentationLODDistance"), Category=InstancedActors)
	double DetailedRepresentationLODDistance = 7000.0;

	/** Force 'Low' representation LOD for all entities further than this distance from a viewer */
	UPROPERTY(EditAnywhere, Meta = (EditCondition = "bOverride_ForceLowRepresentationLODDistance"), Category=InstancedActors)
	double ForceLowRepresentationLODDistance = 27500.0;

	/** Distance in cm from which WPO is disabled */
	UPROPERTY(EditAnywhere, Meta = (EditCondition = "bOverride_WorldPositionOffsetDisableDistance"), Category=InstancedActors)
	int32 WorldPositionOffsetDisableDistance = 5000;

	/**
	 * If bEjectOnActorMoved = true, spawned Actors will have their locations monitored and if moved further than 
	 * ActorEjectionMovementThreshold from their spawn location, they will be 'ejected' from their manager / marked 
	 * as destroyed and transferred to persistence system.
	 */
	UPROPERTY(EditAnywhere, Meta = (EditCondition = "bOverride_bEjectOnActorMoved"), Category=InstancedActors)
	bool bEjectOnActorMoved = false;

	/**
	 * If bEjectOnActorMoved = true, spawned Actors will have their locations monitored and if moved further than 
	 * ActorEjectionMovementThreshold from their spawn location, they will be 'ejected' from their manager / marked 
	 * as destroyed and transferred to persistence system.
	 */
	UPROPERTY(EditAnywhere, Meta = (EditCondition = "bOverride_ActorEjectionMovementThreshold"), Category=InstancedActors)
	float ActorEjectionMovementThreshold = 1.0f;

	/** Can this object ever affect the navigation graph generation for AI etc. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_bCanEverAffectNavigation"), Category=InstancedActors)
	bool bCanEverAffectNavigation = true;

	/** What world partition grid should this instance be placed into. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_OverrideWorldPartitionGrid"), Category=InstancedActors)
	FName OverrideWorldPartitionGrid = TEXT("MainGrid");

	/** Scale the number of entities spawned. It must be between 0.0 and 1.0, for 0% and 100% respectively.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_ScaleEntityCount", ClampMin="0.0", ClampMax="1.0", UIMin="0.8", UIMax="1.0"), Category=InstancedActors)
	float ScaleEntityCount = 1.0f;

	/** Wholesale replace this type of entity with this actor instead. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_ActorClass"), Category=InstancedActors)
	TSubclassOf<AActor> ActorClass;

	/** Turn on or off damage. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_bCanBeDamaged"), Category=InstancedActors)
	bool bCanBeDamaged = true;

	/** Completely disable modifier volumes for this type of instance. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_bIgnoreModifierVolumes"), Category=InstancedActors)
	bool bIgnoreModifierVolumes = false;

	/** By default, modifier volumes will do a bounds intersection test against instances. If this is true, modifiers will instead do a fully-enclosed bounds test */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_bModifierVolumeCheckFullyEnclosed"), Category=InstancedActors)
	bool bModifierVolumeCheckFullyEnclosed = false;
	
	/** Gameplay Tags that can be used in Modifier queries */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Meta = (EditCondition = "bOverride_GameplayTags"), Category=InstancedActors)
	FGameplayTagContainer GameplayTags;

	// Note: Don't forget to implement overriding for new settings in ApplyOverrides and Stringification in ToString
	UPROPERTY(VisibleAnywhere, Category = InstancedActors)
	TArray<FName> AppliedSettingsOverrides;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	uint8 ObjectClassVersion = 0;

	/** Sets up the cached tag containers */
	virtual void PostSerialize(const FArchive& Ar) {}
#endif

	UE_API void ComputeLODDistanceData(double& OutMaxInstanceDistance, double& MaxDrawDistanceScale, float& OutLODDistanceScale) const;

	UE_API bool GetAffectDistanceFieldLighting() const;

	// Apply OverrideSettings to settings not already overridden
	// Note: This is designed for applying settings in reverse order i.e: applying highest priority settings first.
	UE_API virtual void OverrideIfDefault(FConstStructView OverrideSettings, const FName& OverrideSettingsName);

	UE_API virtual FString DebugToString(bool bOverridesOnly = true) const;
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FInstancedActorsSettings> : public TStructOpsTypeTraitsBase2<FInstancedActorsSettings>
{
	enum
	{
		WithPostSerialize = true,
	};
};
#endif // WITH_EDITORONLY_DATA


/** 
 * Per-class settings for instanced actors.
 * 
 * Per-class settings defined in UInstancedActorsProjectSettings::ActorClassSettingsRegistryType are automatically
 * applied to all instances of ActorClass, or its subclasses at runtime. Defining settings separate to the actor class 
 * itself in this way allows actor classes to be instanced with customization, without requiring subclassing to override behavior.
 * 
 * Final compiled settings order for ActorClass is:
 * 
 *    1) Default construted FInstancedActorsSettings
 *    2) UInstancedActorsProjectSettings::DefaultBaseSettingsName
 * 
 *    3) ActorClass::Super's FInstancedActorsClassSettingsBase BaseSettings (if any)
 *         [0]
 *         [1]
 *         [2]
 *         ...
 *    4) ActorClass::Super's OverrideSettings
 *    ..... for all Super's ....
 *
 *    5) ActorClass BaseSettings
 *         [0]
 *         [1]
 *         [2]
 *         ...
 *    6) ActorClass OverrideSettings
 *
 *    7) UInstancedActorsProjectSettings::EnforcedSettingsName
 * 
 * @see FInstancedActorsSettings
 */
USTRUCT()
struct FInstancedActorsClassSettingsBase : public FTableRowBase
{
	GENERATED_BODY()
	
	// @todo There's FDataRegistryId (with UI), but I think it cannot be used here since it has both the registry and name init.
	/** 
	 * Optional ordered list of 'named' settings to apply to instances of ActorClass before applying OverrideSettings.
	 * BaseSettings are applied in order, so the last setting wins (with OverrideSettings having final say).
	 * Note: UInstancedActorsProjectSettings::DefaultBaseSettingsName if specified is effectively inserted into this list at 0
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InstancedActors)
	TArray<FName> BaseSettings;

	/** A virtual function to be overridden by child-types that provide an instance of their specific FInstancedActorsSettings flavor */
	virtual FInstancedStruct MakeOverrideSettings() const { return FInstancedStruct::Make<FInstancedActorsSettings>(); }

};

/** 
 * Generic implementation of FInstancedActorsClassSettingsBase that's using the generic FInstancedActorsSettings to override
 * existing settings. If you want to extend the per-actor-class settings for your project then inhering either from 
 * FInstancedActorsClassSettings and add extra logic and properties to the child type, or inherit from FInstancedActorsClassSettingsBase
 * to provide a project-specific settings struct that inherits FInstancedActorsSettings.
 */
USTRUCT()
struct FInstancedActorsClassSettings : public FInstancedActorsClassSettingsBase
{
	GENERATED_BODY()

	/**
	 * Settings specific to ActorClass instances, applied after / overriding BaseSettings.
	 * Note: UInstancedActorsProjectSettings::EnforcedSettingsName if specified can still override these.
	 */ 
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = InstancedActors)
	FInstancedActorsSettings OverrideSettings;

	virtual FInstancedStruct MakeOverrideSettings() const { return FInstancedStruct::Make<FInstancedActorsSettings>(OverrideSettings); }
};

#undef UE_API
