// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "InstancedActorsTypes.h"
#include "StructUtils/SharedStruct.h"
#include "InstancedActorsStationaryLODBatchProcessor.generated.h"


UCLASS()
class UInstancedActorsStationaryLODBatchProcessor : public UMassProcessor
{
	GENERATED_BODY()
	
public:
	UInstancedActorsStationaryLODBatchProcessor();

protected:
	virtual bool ShouldAllowQueryBasedPruning(const bool bRuntimeMode = true) const override { return false; }
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery LODChangingEntityQuery;
	FMassEntityQuery DirtyVisualizationEntityQuery;

	UPROPERTY(EditDefaultsOnly, Category="Mass", config)
	double DelayPerBulkLOD[(int)EInstancedActorsBulkLOD::MAX];
};
