// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsDebug.h"
#include "InstancedActorsManager.h"
#include "GameplayTagContainer.h"
#include "HierarchicalHashGrid2D.h"
#include "Subsystems/WorldSubsystem.h"
#include "StructUtils/SharedStruct.h"
#include "UObject/ObjectKey.h"
#include "InstancedActorsSubsystem.generated.h"


class AInstancedActorsManager;
class UActorPartitionSubsystem;
class UDataRegistrySubsystem;
class UInstancedActorsModifierVolumeComponent;
class ULevel;
struct FInstancedActorsInstanceHandle;
struct FInstancedActorsManagerHandle;
struct FInstancedActorsModifierVolumeHandle;
class UInstancedActorsProjectSettings;

namespace UE::InstancedActors
{
struct FExemplarActorData;
} // UE::InstancedActors

/**
 * Instanced Actor subsystem used to spawn AInstancedActorsManager's and populate their instance data.
 * It also keeps track of all InstancedActorDatas and can be queried for them
 * @see AInstancedActorsManager
 */
UCLASS(MinimalAPI, BlueprintType)
class UInstancedActorsSubsystem : public UTickableWorldSubsystem
{
	GENERATED_BODY()

public:
	INSTANCEDACTORS_API UInstancedActorsSubsystem();

	INSTANCEDACTORS_API static UInstancedActorsSubsystem* Get(UObject* WorldContextObject);

	INSTANCEDACTORS_API static UInstancedActorsSubsystem& GetChecked(UObject* WorldContextObject);

#if WITH_EDITOR
	/** 
	* Adds an instance of ActorClass at InstanceTransform location by spawning or reusing a AInstancedActorsManager at InstanceTransform's grid cell location.
	* @param AdditionalInstanceTags Gameplay tags that will get appended to class-based FInstancedActorsSettings::GameplayTags
	* @see UInstancedActorsProjectSettings::GridSize
	*/	
	INSTANCEDACTORS_API FInstancedActorsInstanceHandle InstanceActor(TSubclassOf<AActor> ActorClass, FTransform InstanceTransform, ULevel* Level
		, const FGameplayTagContainer& AdditionalInstanceTags = FGameplayTagContainer());

	/** 
	* Adds an instance of ActorClass at InstanceTransform location by spawning or reusing a AInstancedActorsManager at InstanceTransform's grid cell location.
	* * @param AdditionalInstanceTags Gameplay tags that will get appended to class-based FInstancedActorsSettings::GameplayTags
	* @see UInstancedActorsProjectSettings::GridSize
	*/	
	UFUNCTION(BlueprintCallable, Category = InstancedActors)
	INSTANCEDACTORS_API FInstancedActorsInstanceHandle InstanceActor(TSubclassOf<AActor> ActorClass, FTransform InstanceTransform, ULevel* Level
		, const FGameplayTagContainer& AdditionalInstanceTags, TSubclassOf<AInstancedActorsManager> ManagerClass);

	/**
	 * Removes all instance data for InstanceHandle.	
	 *
	 * This simply adds this instance to a FreeList which incurs extra cost to process at runtime before instance spawning. 
	 *
	 * @param InstanceHandle         Instance to remove
	 * @param bDestroyManagerIfEmpty If true, and InstanceHandle is the last instance in its manager, destroy the 
	 *                               now-empty manager. This implicitly clears the FreeList which is stored per-manager. 
	 *                               Any subsequent instance adds will then create a fresh manager.
	 * @see IA.CompactInstances console command
     */
	UFUNCTION(BlueprintCallable, Category = InstancedActors)
	INSTANCEDACTORS_API bool RemoveActorInstance(const FInstancedActorsInstanceHandle& InstanceHandle, bool bDestroyManagerIfEmpty = true);
#endif // WITH_EDITOR

	INSTANCEDACTORS_API void ForEachManager(const FBox& QueryBounds, TFunctionRef<bool(AInstancedActorsManager&)> InOperation
		, TSubclassOf<AInstancedActorsManager> ManagerClass = AInstancedActorsManager::StaticClass()) const;
	INSTANCEDACTORS_API void ForEachModifierVolume(const FBox& QueryBounds, TFunctionRef<bool(UInstancedActorsModifierVolumeComponent&)> InOperation) const;
	INSTANCEDACTORS_API void ForEachInstance(const FBox& QueryBounds, TFunctionRef<bool(const FInstancedActorsInstanceHandle&
		, const FTransform&, FInstancedActorsIterationContext&)> InOperation) const;

	/** 
	 * Checks whether there are any instanced actors representing ActorClass or its subclasses inside QueryBounds.
	 * The check doesn't differentiate between hydrated and dehydrated actors (i.e. whether there's an actor instance 
	 * associated with the instance or not).
	 * @param bTestActorsIfSpawned if true then when an instance is found to overlap given bounds, and it has an actor
	 *	spawned associated with it, then the actor itself will be tested against the bounds for more precise test.
	 * @param AllowedLODs if provided will be used to filter out InstancedActorData that are at LOD not matching the flags in AllowedLODs
	 */
	INSTANCEDACTORS_API bool HasInstancesOfClass(const FBox& QueryBounds, TSubclassOf<AActor> ActorClass, const bool bTestActorsIfSpawned = false
		, const EInstancedActorsBulkLODMask AllowedLODs = EInstancedActorsBulkLODMask::All) const;

	INSTANCEDACTORS_API FInstancedActorsManagerHandle AddManager(AInstancedActorsManager& Manager);
	void RemoveManager(FInstancedActorsManagerHandle ManagerHandle);

	FInstancedActorsModifierVolumeHandle AddModifierVolume(UInstancedActorsModifierVolumeComponent& ModifierVolume);
	void RemoveModifierVolume(FInstancedActorsModifierVolumeHandle ModifierVolumeHandle);

	/** Adds ManagerHandle to PendingManagersToSpawnEntities for later processing in Tick -> ExecutePendingDeferredSpawnEntitiesRequests */
	void RequestDeferredSpawnEntities(FInstancedActorsManagerHandle ManagerHandle);

	/**
	 * Removes ManagerHandle from PendingManagersToSpawnEntities if present 
	 * @return	true if ManagerHandle was present in PendingManagersToSpawnEntities and subsequently removed. False otherwise (list is empty
	 *			or didn't contain ManagerHandle)
	 */
	bool CancelDeferredSpawnEntitiesRequest(FInstancedActorsManagerHandle ManagerHandle);

	/**
	 * Calls AInstancedActorsManager::InitializeModifyAndSpawnEntities for all pending managers in PendingManagersToSpawnEntities
	 * added via RequestDeferredSpawnEntities.
	 * @param	StopAfterSeconds	If < INFINITY, requests processing will stop after this time, leaving remaining requests for the next 
	 *								ExecutePendingDeferredSpawnEntitiesRequests to continue.
	 */
	bool ExecutePendingDeferredSpawnEntitiesRequests(double StopAfterSeconds = INFINITY);

	/** Return true if any deferred spawn entities requests are pending execution by the next ExecutePendingDeferredSpawnEntitiesRequests */
	INSTANCEDACTORS_API bool HasPendingDeferredSpawnEntitiesRequests() const;

	/**
	 * Retrieves existing or spawns a new ActorClass for introspecting exemplary instance data.
	 *
	 * Actors are spawned into ExemplarActorWorld, a separated 'inactive' UWorld, to ensure no conflict or modifications in
	 * the main game world.
	 *
	 * These 'exemplar' actors are fully constructed, including BP construction scripts up to (but not including) BeginPlay.
	 */
	INSTANCEDACTORS_API TSharedRef<UE::InstancedActors::FExemplarActorData> GetOrCreateExemplarActor(TSubclassOf<AActor> ActorClass);
	
	/**
	 * Removes exemplar actor class from the map
	 */
	INSTANCEDACTORS_API void UnregisterExemplarActorClass(TSubclassOf<AActor> ActorClass);

	/** 
	 * Compiles and caches finalized settings for ActorClass based off FInstancedActorsClassSettingsBase found in 
	 * UInstancedActorsProjectSettings::ActorClassSettingsRegistryType data registry, for ActorClass and it's 
	 * inherited super classes.
	 *
	 * Note: FInstancedActorsClassSettingsBase are indexed by class FName (*not* the full path) in the data registry, 
	 *       for quick lookup in CompileSettingsForActorClass. This means unique class names must be used for 
	 *       per-class settings.
	 */
	INSTANCEDACTORS_API FSharedStruct GetOrCompileSettingsForActorClass(TSubclassOf<AActor> ActorClass);

	/**
	 * Returns true if ActorClass has a matching FInstancedActorsClassSettingsBase entry in 
	 * UInstancedActorsProjectSettings::ActorClassSettingsRegistryType data registry.
	 * Note: This relies on the registry being loaded at the time of calling i.e: in editor the registry must be
	 *       set to preload in editor.
	 * @param bIncludeSuperClasses	If true and ActorClass doesn't have a direct entry in ActorClassSettingsRegistryType,
	 * 								ActorClass's super classes will be successively tried instead.
	 */
	bool DoesActorClassHaveRegisteredSettings(TSubclassOf<AActor> ActorClass, bool bIncludeSuperClasses = true);

	/**
	 * Adds InstanceHandle to a list of pending instances that require an explicit representation update e.g: to remove/add ISMCs when a
	 * replicated actor is spawned / despawned. This ensures their representation is updated even if the are currently in a non-detailed
	 * bulk LOD.
	 * @see UInstancedActorsStationaryLODBatchProcessor
	 */
	void MarkInstanceRepresentationDirty(FInstancedActorsInstanceHandle InstanceHandle);

	/**
	 * Copies the current list of instances requiring explicit representation updates (added via MarkInstanceRepresentationDirty) to
	 * OutInstances and clears the internal list. This 'consumes' these pending instances with the assumption they will then receive a
	 * representation update by the calling code.
	 * @see MarkInstanceRepresentationDirty, UInstancedActorsStationaryLODBatchProcessor
	 */
	INSTANCEDACTORS_API void PopAllDirtyRepresentationInstances(TArray<FInstancedActorsInstanceHandle>& OutInstances);

	INSTANCEDACTORS_API virtual FInstancedActorsVisualizationDesc CreateVisualDescriptionFromActor(const AActor& ExemplarActor) const;
	/*
	* Called when an additional/alternate VisualizationDesc is registered. Override to make custom modifications to the visual representation
	*/
	virtual void ModifyVisualDescriptionForActor(const TNotNull<AActor*> ExemplarActor, FInstancedActorsVisualizationDesc& InOutVisualization) const {}

	//~ Begin UTickableWorldSubsystem Overrides
	INSTANCEDACTORS_API virtual void Tick(float DeltaTime) override;
	INSTANCEDACTORS_API virtual TStatId GetStatId() const override;
	//~ End UTickableWorldSubsystem Overrides

	//~ Begin USubsystem Overrides
	INSTANCEDACTORS_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	INSTANCEDACTORS_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	INSTANCEDACTORS_API virtual void Deinitialize() override;
	//~ End USubsystem Overrides

	struct FNextTickSharedFragment
	{
		FSharedStruct SharedStruct;
		double NextTickTime = 0;

		bool operator<(const FNextTickSharedFragment& Other) const
		{
			return NextTickTime < Other.NextTickTime;
		}
	};

	TArray<UInstancedActorsSubsystem::FNextTickSharedFragment>& GetTickableSharedFragments();
	void UpdateAndResetTickTime(TConstStructView<FInstancedActorsDataSharedFragment> InstancedActorsDataSharedFragment);

	TSubclassOf<AInstancedActorsManager> GetInstancedActorsManagerClass() const 
	{ 
		return InstancedActorsManagerClass; 
	}

	/**
	 * Adds InstancedActorsData to a list of pending IAD which contains entities that should be destroyed
	 * @see UInstancedActorsRuntimeRemovalProcessor, UInstancedActorsData::RuntimeRemoveInstances
	 */
	void MarkIADDirtyWithEntitiesPendingDestruction(UInstancedActorsData& InstancedActorsData);
	/**
	 * Moves the current list of IAD with entities pending destruction (added via MarkIadDirtyWithEntitiesPendingDestruction) to
	 * OutInstancedActorsData and clears the internal list. This 'consumes' these pending IAD with the assumption the entities pending
	 * destruction it contains will get destroyed by next command flush.
	 * @see UInstancedActorsRuntimeRemovalProcessor, UInstancedActorsData::RuntimeRemoveInstances
	 */
	INSTANCEDACTORS_API void PopAllDirtyIADWithEntitiesPendingDestruction(TArray<TWeakObjectPtr<UInstancedActorsData>>& OutInstancedActorsData);
	bool ContainsDirtyIADWithEntitiesPendingDestruction() const 
	{ 
		return DirtyIadWithEntitiesPendingDestruction.Num() > 0; 
	}

protected:
	/** 
	 * Fetches all registered FInstancedActorsDataSharedFragment from the EntityManager and adds the missing ones to SortedSharedFragments
	 * @param InstancedActorsDataSharedFragment optionally the function can check if given shared fragment is amongst 
	 *	the newly added fragments
	 * @param returns whether InstancedActorsDataSharedFragment has been found, or `true` if that param is not provided. 
	 */
	bool RegisterNewSharedFragmentsInternal(TConstStructView<FInstancedActorsDataSharedFragment> InstancedActorsDataSharedFragment = TConstStructView<FInstancedActorsDataSharedFragment>());

	/** The container storing a sorted queue of FSharedStruct instances, ordered by the NextTickTime */
	TArray<FNextTickSharedFragment> SortedSharedFragments;

	TSharedPtr<FMassEntityManager> EntityManager;

	UPROPERTY(Transient)
	TObjectPtr<const UInstancedActorsProjectSettings> ProjectSettings;

	UPROPERTY(Transient)
	TObjectPtr<const UDataRegistrySubsystem> DataRegistrySubsystem;

	UPROPERTY(Transient)
	TObjectPtr<UActorPartitionSubsystem> ActorPartitionSubsystem;

	UPROPERTY()
	TSubclassOf<AInstancedActorsManager> InstancedActorsManagerClass;

	// Spatially indexed managers. TSparseArray used for stable indices which can be spatially indexed by THierarchicalHashGrid2D
	// @todo Managers should be indexable by cell coord hashes within their levels, we could leverage this for more efficient spatial
	//       indexing by keeping a cellcoord hash map per tile
	TSparseArray<TWeakObjectPtr<AInstancedActorsManager>> Managers;
	using FManagersHashGridType = THierarchicalHashGrid2D</*Levels*/3, /*LevelRatio*/4, /*ItemIDType*/FInstancedActorsManagerHandle>;
	FManagersHashGridType ManagersHashGrid;

	// Spatially indexed modifier volumes. TSparseArray used for stable indices which can be spatially indexed by THierarchicalHashGrid2D
	TSparseArray<TWeakObjectPtr<UInstancedActorsModifierVolumeComponent>> ModifierVolumes;
	using FModifierVolumesHashGridType = THierarchicalHashGrid2D</*Levels*/3, /*LevelRatio*/4, /*ItemIDType*/FInstancedActorsModifierVolumeHandle>;
	FModifierVolumesHashGridType ModifierVolumesHashGrid;

	// FIFO queue of Managers pending deferred entity spawning in Tick. Enqueued in RequestDeferredSpawnEntities
	TArray<FInstancedActorsManagerHandle> PendingManagersToSpawnEntities;

	// Instances whose representation is explicitly dirty, e.g: due to actor spawn / despawn replication, requiring immediate representation 
	// processing even out of 'detailed' representation processing range.
	TArray<FInstancedActorsInstanceHandle> DirtyRepresentationInstances;

	mutable FCriticalSection DirtyIadWithEntitiesPendingDestructionCS;
	TArray<TWeakObjectPtr<UInstancedActorsData>> DirtyIadWithEntitiesPendingDestruction;

#if WITH_INSTANCEDACTORS_DEBUG
	TMap<TObjectKey<AInstancedActorsManager>, FBox> DebugManagerBounds;
	TMap<TObjectKey<UInstancedActorsModifierVolumeComponent>, FBox> DebugModifierVolumeBounds;
#endif

	// Cached finalized / flattened FInstancedActorsClassSettingsBase for GetOrCompileSettingsForActorClass requested ActorClass.
	// Built via CompileSettingsForActorClass and cached in GetOrCompileSettingsForActorClass
	TMap<TWeakObjectPtr<UClass>, FSharedStruct> PerActorClassSettings;

	// Called in GetOrCompileSettingsForActorClass to compile finalized settings for ActorClass based off 
	// FInstancedActorsClassSettingsBase found in UInstancedActorsProjectSettings::ActorClassSettingsRegistryType
	// data registry, for ActorClass and it's inherited super classes.
	FSharedStruct CompileSettingsForActorClass(TSubclassOf<AActor> ActorClass) const;

#if WITH_EDITOR
	void HandleRefreshSettings(IConsoleVariable* InCVar);
#endif

	// Inactive UWorld housing lazily create exemplar actors for instance actor classes
	// @see GetOrCreateExemplarActor
	UPROPERTY(Transient)
	TObjectPtr<UWorld> ExemplarActorWorld;

	// Lazily created exemplar actors for instance actor classes
	// @see GetOrCreateExemplarActor
	TMap<TObjectKey<const UClass>, TWeakPtr<UE::InstancedActors::FExemplarActorData>> ExemplarActors;

	UPROPERTY(Transient)
	TObjectPtr<const UScriptStruct> SettingsType;
};
