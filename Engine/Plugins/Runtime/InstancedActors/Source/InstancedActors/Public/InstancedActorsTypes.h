// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "GameplayTagContainer.h"
#include "ISMPartition/ISMComponentDescriptor.h"
#include "MassEntityTypes.h"
#include "MassRepresentationTypes.h"
#include "MassSkinnedMeshRepresentationTypes.h"
#include "InstancedActorsIndex.h"
#include "InstancedActorsTypes.generated.h"

#define UE_API INSTANCEDACTORS_API


INSTANCEDACTORS_API DECLARE_LOG_CATEGORY_EXTERN(LogInstancedActors, Log, All)

struct FStaticMeshInstanceVisualizationDesc;
struct FStreamableHandle;
struct FMassISMCSharedData;
class UInstancedActorsData;
class UWorld;
class UServerInstancedActorsSpawnerSubsystem;
class UClientInstancedActorsSpawnerSubsystem;
class UMassActorSpawnerSubsystem;
class UInstancedActorsSubsystem;

enum class EInstancedActorsBulkLOD : uint8
{
	Detailed, // this will make Mass calculate LOD individually for every instance
	Medium,
	Low,
	Off,
	MAX
};

enum class EInstancedActorsBulkLODMask : uint8
{
	None = 0,

	Detailed = 1 << int(EInstancedActorsBulkLOD::Detailed),
	Medium = 1 << int(EInstancedActorsBulkLOD::Medium),
	Low = 1 << int(EInstancedActorsBulkLOD::Low),
	Off = 1 << int(EInstancedActorsBulkLOD::Off),

	NotDetailed = Medium | Low | Off,
	All = 0xFF
};

enum class EInstancedActorsFragmentFlags : uint8
{
    None = 0,

    Replicated = 1 << 0,
    Persisted = 1 << 1,

    All = 0xFF,
};
ENUM_CLASS_FLAGS(EInstancedActorsFragmentFlags);

namespace UE::InstancedActors::Utils
{
	INSTANCEDACTORS_API TSubclassOf<UMassActorSpawnerSubsystem> DetermineActorSpawnerSubsystemClass(const UWorld& World);
	INSTANCEDACTORS_API UServerInstancedActorsSpawnerSubsystem* GetServerInstancedActorsSpawnerSubsystem(const UWorld& World);
	INSTANCEDACTORS_API UClientInstancedActorsSpawnerSubsystem* GetClientInstancedActorsSpawnerSubsystem(const UWorld& World);
	/** 
	 * Calls either GetServerInstancedActorsSpawnerSubsystem or GetClientInstancedActorsSpawnerSubsystem, depending on 
	 * given UWorld's net mode.
	 */
	INSTANCEDACTORS_API UMassActorSpawnerSubsystem* GetActorSpawnerSubsystem(const UWorld& World);
	INSTANCEDACTORS_API UInstancedActorsSubsystem* GetInstancedActorsSubsystem(const UWorld& World);
}

// FInstancedActorsTagSet -> FInstancedActorsTagSet
/** An immutable hashed tag container used to categorize / partition instances */
USTRUCT(BlueprintType)
struct FInstancedActorsTagSet
{
	GENERATED_BODY()

	FInstancedActorsTagSet() = default;
	UE_API FInstancedActorsTagSet(const FGameplayTagContainer& InTags);

	bool IsEmpty() const { return Tags.IsEmpty(); }

	const FGameplayTagContainer& GetTags() const { return Tags; }
	
	UE_API void CalcHash();

	bool operator==(const FInstancedActorsTagSet& OtherTagSet) const
	{
		if (Hash.IsSet() && OtherTagSet.Hash.IsSet() && Hash.GetValue() != OtherTagSet.Hash.GetValue())
		{
			return false;
		}

		if (Tags.Num() != OtherTagSet.Tags.Num())
		{
			return false;
		}

		// Tags member will have already been constructed from a sorted TArray
		for (int32 i = 0; i < Tags.Num(); ++i)
		{
			if (Tags.GetGameplayTagArray()[i] != OtherTagSet.Tags.GetGameplayTagArray()[i])
			{
				return false;
			}
		}

		return true;
	}
	
private:

	UPROPERTY(VisibleAnywhere, Category=InstancedActors)
	FGameplayTagContainer Tags;

	UPROPERTY(Transient)
	TOptional<uint32> Hash;
};


/**
 * ISMC descriptions for instances 'visualization', allowing instances to define multiple
 * potential visualizations / ISMC sets: e.g: 'with berries', 'without berries'.
 */
USTRUCT()
struct FInstancedActorsVisualizationDesc
{
	GENERATED_BODY()

	FInstancedActorsVisualizationDesc() = default;
	UE_API explicit FInstancedActorsVisualizationDesc(const FInstancedActorsSoftVisualizationDesc& SoftVisualizationDesc);

	/**
	 * Array of Instanced Static Mesh Component descriptors. An ISMC will be created for each of these, using the specified mesh, material,
	 * collision settings etc. Instanced Actors using this visualization will add an instance to each of these, allowing for composite mesh
	 * visualizations for a single actor instance e.g: a car with separate body and wheel meshes all instanced together.
	 */
	UPROPERTY(VisibleAnywhere, Category = InstancedActors)
	TArray<FISMComponentDescriptor> ISMComponentDescriptors;

	UPROPERTY(VisibleAnywhere, Category = InstancedActors)
	TArray<float> CustomDataFloats;

	using FAdditionalSetupStepsFunction = TFunctionRef<void(const AActor& /*ExemplarActor*/, FISMComponentDescriptor& /*NewISMComponentDescriptor*/, FInstancedActorsVisualizationDesc& /*OutVisualization*/)>;	
	using FVisualizationDescSetupFunction = TFunctionRef<void(const AActor& /*ExemplarActor*/, FInstancedActorsVisualizationDesc& /*OutVisualization*/)>;
	
	UE_DEPRECATED(5.6, "Using FAdditionalSetupStepsFunction which takes a FISMComponentDescriptor& parameter is deprecated. Use FVisualizationDescSetupFunction instead, since FInstancedActorsVisualizationDesc already contains the built TArray of FISMComponentDescriptor")
	static UE_API FInstancedActorsVisualizationDesc FromActor(const AActor& ExemplarActor, const FAdditionalSetupStepsFunction& AdditionalSetupSteps);
	
	/**
	 * Helper function to deduce appropriate instanced static mesh representation for an ActorClass exemplar actor.
	 * @param ExemplarActor A fully constructed exemplar actor, including BP construction scripts up to (but not including) BeginPlay.
	 * @param AdditionalSetupFunction function called once a new ISMComponentDescriptor is created, allowing custom code to 
	 *	perform additional set up steps. Note that this function will get called only if ExemplarActor has a 
	 *	StaticMeshComponent with a valid StaticMesh configured.
	 * @see UInstancedActorsSubsystem::GetOrCreateExemplarActor
	 */	
	static UE_API FInstancedActorsVisualizationDesc FromActor(const AActor& ExemplarActor, const FVisualizationDescSetupFunction& AdditionalSetupFunction = [](const AActor& /*ExemplarActor*/, FInstancedActorsVisualizationDesc& /*OutVisualization*/){});

	UE_API FStaticMeshInstanceVisualizationDesc ToMassVisualizationDesc() const;

	friend inline uint32 GetTypeHash(const FInstancedActorsVisualizationDesc& InDesc)
	{
		uint32 Hash = 0;
		for (const FISMComponentDescriptor& InstancedMesh : InDesc.ISMComponentDescriptors)
		{
			Hash = HashCombine(Hash, GetTypeHash(InstancedMesh));
		}
		return Hash;
	}
};


/**
 * Soft-ptr variant of FInstancedActorsVisualizationDesc for defining visualization assets to async load.
 */
USTRUCT()
struct FInstancedActorsSoftVisualizationDesc
{
	GENERATED_BODY()

	FInstancedActorsSoftVisualizationDesc() = default;
	UE_API explicit FInstancedActorsSoftVisualizationDesc(const FInstancedActorsVisualizationDesc& VisualizationDesc);

	/**
	 * Array of Instanced Static Mesh Component descriptors. An ISMC will be created for each of these, using the specified mesh, material,
	 * collision settings etc. Instanced Actors using this visualization will add an instance to each of these, allowing for composite mesh
	 * visualizations for a single actor instance e.g: a car with separate body and wheel meshes all instanced together.
	 */
	UPROPERTY(VisibleAnywhere, Category = InstancedActors)
	TArray<FSoftISMComponentDescriptor> ISMComponentDescriptors;

	UE_API void GetAssetsToLoad(TArray<FSoftObjectPath>& OutAssetsToLoad) const;
};


/** Runtime ISMC tracking for a given 'visualization' (alternate ISMC set) for instances */
USTRUCT()
struct FInstancedActorsVisualizationInfo
{
	GENERATED_BODY()

	/**
	 * Returns true if this visualization was added view UInstancedActorsData::AddVisualizationAsync and streaming is still in-progress.
	 * Once streaming completes, Desc, ISMComponents and MassStaticMeshDescIndex will be valid and this returns false.
	 * Note: Until streaming completes, Desc, ISMComponents & MassStaticMeshDescIndex will all be defayult values / unset.
	 */
	inline bool IsAsyncLoading() const { return AssetLoadHandle.IsValid(); }

	/**
	 * Cached specification for this visualization, defining ISMCs to create.
	 * Note: For visualizations added via UInstancedActorsData::AddVisualizationAsync using an FInstancedActorsSoftVisualizationDesc
	 *       soft pointer descriptor: whilst IsAsyncLoading() = true, this will be default constructed. Once the async load completes, the
	 *       soft visualization decriptor will then be resolved to this hard pointer decriptor.
	 */
	UPROPERTY(VisibleAnywhere, Category = InstancedActors)
	FInstancedActorsVisualizationDesc VisualizationDesc;

	/**
	 * Instanced Static Mesh Components created from VisualizationDesc.ISMComponentDescriptors specs.
	 *
	 * Note: If IsAsyncLoading() = true, this will be empty until the load completes and ISMCs are created.
	 */
	UPROPERTY(VisibleAnywhere, Category = InstancedActors)
	TArray<TObjectPtr<UInstancedStaticMeshComponent>> ISMComponents;

	/**
	 * Handle to registration of ISMComponents with UMassRepresentationSubsystem via
	 * UMassRepresentationSubsystem::AddVisualDescWithISMComponents.
	 *
	 * Note: If IsAsyncLoading() = true, this will be invalid until the load completes and ISMCs are created and registered.
	 */
	UPROPERTY(VisibleAnywhere, Category = InstancedActors)
	FStaticMeshInstanceVisualizationDescHandle MassStaticMeshDescHandle;

	// If this visualization was added with UInstancedActorsData::AddVisualizationAsync, this will be set to the async streaming request 
	// until streaming is complete, whereupon this handle is cleared.
	TSharedPtr<FStreamableHandle> AssetLoadHandle;

	/** Used to track version of data used to create CollisionIndexToEntityIndexMap */
	mutable uint16 CachedTouchCounter = 0;
	/**
	 * Valid as long as Mass visualization data indicated by MassStaticMeshDescIndex has ComponentInstanceIdTouchCounter
	 * equal to CachedTouchCounter.
	 */
	mutable TArray<int32> CollisionIndexToEntityIndexMap;
};


USTRUCT()
struct FInstancedActorsMeshSwitchFragment : public FMassFragment
{
	GENERATED_BODY()

	// The pending Mass static mesh representation index we want to switch to.
	// @see UInstancedActorsVisualizationSwitcherProcessor
	UPROPERTY()
	FStaticMeshInstanceVisualizationDescHandle NewStaticMeshDescHandle;
	
	// The pending Mass skinned mesh representation index we want to switch to.
	// @see UInstancedActorsVisualizationSwitcherProcessor
	UPROPERTY()
	FSkinnedMeshInstanceVisualizationDescHandle NewSkinnedMeshDescHandle;
};


USTRUCT(BlueprintType)
struct FInstancedActorsManagerHandle
{
	GENERATED_BODY()

public:

	FInstancedActorsManagerHandle() = default;
	FInstancedActorsManagerHandle(const int32 InManagerID) : ManagerID(InManagerID) {}

	inline bool IsValid() const
	{
		return ManagerID != INDEX_NONE;
	}

	inline void Reset()
	{
		ManagerID = INDEX_NONE;
	}

	int32 GetManagerID() const
	{
		return ManagerID;
	}

	bool operator==(const FInstancedActorsManagerHandle&) const = default;

private:

	UPROPERTY(Transient)
	int32 ManagerID = INDEX_NONE;
};

USTRUCT(BlueprintType)
struct FInstancedActorsModifierVolumeHandle
{
	GENERATED_BODY()

public:

	FInstancedActorsModifierVolumeHandle() = default;
	FInstancedActorsModifierVolumeHandle(const int32 InModifierVolumeID) : ModifierVolumeID(InModifierVolumeID) {}

	int32 GetModifierVolumeID() const
	{
		return ModifierVolumeID;
	}

	bool operator==(const FInstancedActorsModifierVolumeHandle&) const = default;

private:

	UPROPERTY(Transient)
	int32 ModifierVolumeID = INDEX_NONE;
};

/**
 * Note that we don't really need this type to be a shared fragment. It's used to create FSharedStructs pointing at
 * UInstancedActorsData and this data is fetched from MassEntityManager by UInstancedActorsStationaryLODBatchProcessor.
 * @todo This will be addressed in the future by refactoring where this data is stored and how it's used.
 */
USTRUCT()
struct FInstancedActorsDataSharedFragment : public FMassSharedFragment
{
	GENERATED_BODY()

	UPROPERTY(Transient)
	TWeakObjectPtr<UInstancedActorsData> InstanceData;

	EInstancedActorsBulkLOD BulkLOD = EInstancedActorsBulkLOD::MAX;

	// Set to true when the IAD has despawned its entities
	bool bInstancedActorsDataDespawnedEntities = false;

	double LastTickTime = 0.0;
};

USTRUCT()
struct FInstancedActorsFragment : public FMassFragment
{
	GENERATED_BODY()

	// InstancedActorData owning the given entity
	UPROPERTY(Transient)
	TWeakObjectPtr<UInstancedActorsData> InstanceData;

	// The fixed index of this 'instance' into InstanceData
	UPROPERTY()
	FInstancedActorsInstanceIndex InstanceIndex;
};

#undef UE_API
