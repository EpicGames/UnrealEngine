// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassRepresentationProcessor.h"
#include "InstancedActorsVisualizationProcessor.generated.h"


/**
 * Tag required by Instanced Actors Visualization Processor to process given archetype. Removing the tag allows to support
 * disabling of processing for individual entities of given archetype.
 */
USTRUCT()
struct FInstancedActorsVisualizationProcessorTag : public FMassTag
{
	GENERATED_BODY();
};

UCLASS()
class UInstancedActorsVisualizationProcessor : public UMassVisualizationProcessor
{
	GENERATED_BODY()

protected:
	UInstancedActorsVisualizationProcessor();
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
};
