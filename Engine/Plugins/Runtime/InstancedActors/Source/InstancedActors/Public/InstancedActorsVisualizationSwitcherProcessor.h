// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassRepresentationTypes.h"
#include "MassSkinnedMeshRepresentationTypes.h"
#include "MassEntityQuery.h"
#include "InstancedActorsVisualizationSwitcherProcessor.generated.h"


struct FMassRepresentationFragment;

/**
 * Executes on entities with FInstancedActorsMeshSwitchFragment's, processing them as `pending requests` to switch to
 * the specified NewStaticMeshDescHandle, then removing the fragments once complete
 */
UCLASS(MinimalAPI)
class UInstancedActorsVisualizationSwitcherProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UInstancedActorsVisualizationSwitcherProcessor();

	// Switch EntityHandle to NewStaticMeshDescHandle by removing any current / previous ISMC instances for the current StaticMeshDescIHandle
	// and setting RepresentationFragment.PrevRepresentation = EMassRepresentationType::None to let the subsequent
	// UMassStationaryISMSwitcherProcessor see that a new instance needs to be added for the now set NewStaticMeshDescHandle
	INSTANCEDACTORS_API static void SwitchEntityMeshDesc(FMassInstancedStaticMeshInfoArrayView& ISMInfosView, FMassRepresentationFragment& RepresentationFragment, FMassEntityHandle EntityHandle, FStaticMeshInstanceVisualizationDescHandle NewStaticMeshDescHandle);

	// Switch EntityHandle to NewMeshDescHandle by removing any current / previous ISMC instances for the current MeshDescIHandle
	// and setting RepresentationFragment.PrevRepresentation = EMassRepresentationType::None to let the subsequent
	// UMassStationaryISMSwitcherProcessor see that a new instance needs to be added for the now set NewMeshDescHandle
	INSTANCEDACTORS_API static void SwitchEntitySkinnedMeshDesc(FMassInstancedSkinnedMeshInfoArrayView& MeshInfosView, FMassRepresentationFragment& RepresentationFragment, FMassEntityHandle EntityHandle, FSkinnedMeshInstanceVisualizationDescHandle NewMeshDescHandle);
protected:
	virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};
