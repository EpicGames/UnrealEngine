// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStationaryDistanceVisualizationTrait.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "InstancedActorsVisualizationTrait.generated.h"


class UInstancedActorsData;

/** 
 * Subclass of UMassStationaryVisualizationTrait which forces required settings for instanced actor entities and overrides
 * FMassRepresentationFragment.StaticMeshDescHandle to use a custom registered Visualization which reuses InstanceData's 
 * ISMComponents via UMassRepresentationSubsystem::AddVisualDescWithISMComponent.
 * Note that the trait is marked to not show up in class selection drop-downs. The reason is that this trait is supposed
 * to be used internally by InstancedActors and is never expected to be a part of a user-authored entity config.
 */
UCLASS(MinimalAPI, HideDropdown)
class UInstancedActorsVisualizationTrait : public UMassStationaryDistanceVisualizationTrait
{
	GENERATED_BODY()

public:

	INSTANCEDACTORS_API UInstancedActorsVisualizationTrait(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	INSTANCEDACTORS_API virtual void InitializeFromInstanceData(UInstancedActorsData& InInstanceData);

	//~ Begin UInstancedActorsVisualizationTrait Overrides
	INSTANCEDACTORS_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
	//~ End UInstancedActorsVisualizationTrait Overrides

protected:
	
	UPROPERTY(Transient)
	TWeakObjectPtr<UInstancedActorsData> InstanceData;
};
