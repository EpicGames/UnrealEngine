// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InstancedActorsIndex.h"
#include "MassActorSpawnerSubsystem.h"
#include "ServerInstancedActorsSpawnerSubsystem.generated.h"

#define UE_API INSTANCEDACTORS_API


struct FMassEntityManager;
class UInstancedActorsComponent;
struct FInstancedActorsInstanceHandle;
struct FActorSpawnParameters;

/**
 * Dedicated UMassActorSpawnerSubsystem subclass handling server-side Actor spawning for InstancedActor.
 * The main responsibility is InstancedActors-specific setup of newly spawned actors, including configuring
 * UInstancedActorsComponent instanced a newly spawned actor hosts. 
 */
UCLASS(MinimalAPI)
class UServerInstancedActorsSpawnerSubsystem : public UMassActorSpawnerSubsystem
{
	GENERATED_BODY()

public:
	
	/**
	 * Called by UInstancedActorsComponent::InitializeComponent to provide an opportunity, before BeginPlay, to
	 * catch Instanced Actors we're spawning, to set their FMassEntityHandle association.
	 */
	UE_API void OnInstancedActorComponentInitialize(UInstancedActorsComponent& InstancedActorComponent) const;

protected:

	//~ Begin USubsystem Overrides
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual bool ShouldCreateSubsystem(UObject* Outer) const override;
	UE_API virtual bool ReleaseActorToPool(AActor* Actor) override;
	//~ Begin USubsystem Overrides
	
	//~ Begin UMassActorSpawnerSubsystem Overrides
	UE_API virtual ESpawnRequestStatus SpawnActor(const FSpawnArgs& Args, FActorSpawnParameters& InOutSpawnParameters) const override;
    //~ End UMassActorSpawnerSubsystem Overrides

	TSharedPtr<FMassEntityManager> EntityManager;

	// Set during SpawnActor and cleared once complete, to 'catch' UInstancedActorComponents initializing during the actor 
	// spawn, matching their Owner to TransientActorBeingSpawned in OnInstancedActorComponentInitialize, to test if the component 
	// was 'spawned by Instanced Actors'.
	// If so, we call UInstancedActorsComponent::InitializeComponentForInstance and pass along TransientActorSpawningEntity
	// as the Mass entity 'owning' this spawned actor.
	UPROPERTY(Transient)
	mutable TObjectPtr<AActor> TransientActorBeingSpawned = nullptr;

	UPROPERTY(Transient)
	mutable FInstancedActorsInstanceHandle TransientActorSpawningInstance;
};

#undef UE_API
