// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsEditorModule.h"
#include "InstancedActorsSubsystem.h"
#include "InstancedActorsManager.h"
#include "InstancedActorsData.h"
#include "InstancedActorsIteration.h"
#include "InstancedActorsSettings.h"
#include "Modules/ModuleManager.h"
#include "LevelEditor.h"
#include "Engine/World.h"
#include "ScopedTransaction.h"
#include "ActorFactories/ActorFactory.h"


#define LOCTEXT_NAMESPACE "InstancedActorsEditor"

DEFINE_LOG_CATEGORY_STATIC(LogInstancedActorsEditor, Log, All);

IMPLEMENT_MODULE(FInstancedActorsEditorModule, InstancedActorsEditor)

void FInstancedActorsEditorModule::StartupModule()
{
	ResetConversionDelegates();
	AddLevelViewportMenuExtender();
}

void FInstancedActorsEditorModule::ShutdownModule()
{
	// Cleanup menu extenstions
	RemoveLevelViewportMenuExtender();
}

void FInstancedActorsEditorModule::ResetConversionDelegates()
{
	ActorToIADelegate.BindRaw(this, &FInstancedActorsEditorModule::ConvertActorsToIAsUIAction);
	IAToActorDelegate.BindRaw(this, &FInstancedActorsEditorModule::ConvertIAsToActorsUIAction);

	ActorToIAFormatLabel = LOCTEXT("ConvertSelectedActorsToIAsText", "Convert {0} to Instanced Actors");
	IAToActorFormatLabel = LOCTEXT("ConvertSelectedIAsToActorsText", "Convert {0}'s instances back to Actors");
}

void FInstancedActorsEditorModule::AddLevelViewportMenuExtender()
{
	if (!IsRunningGame())
	{
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");
		TArray<FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors>& MenuExtenders = LevelEditorModule.GetAllLevelViewportContextMenuExtenders();

		MenuExtenders.Add(FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors::CreateRaw(this, &FInstancedActorsEditorModule::CreateLevelViewportContextMenuExtender));
		LevelViewportExtenderHandle = MenuExtenders.Last().GetHandle();
	}
}

void FInstancedActorsEditorModule::RemoveLevelViewportMenuExtender()
{
	if (LevelViewportExtenderHandle.IsValid())
	{
		FLevelEditorModule* LevelEditorModule = FModuleManager::Get().GetModulePtr<FLevelEditorModule>("LevelEditor");
		if (LevelEditorModule)
		{
			LevelEditorModule->GetAllLevelViewportContextMenuExtenders().RemoveAll([this](const FLevelEditorModule::FLevelViewportMenuExtender_SelectedActors& In)
				{ 
					return In.GetHandle() == LevelViewportExtenderHandle; 
				});
		}
		LevelViewportExtenderHandle.Reset();
	}
}

TSharedRef<FExtender> FInstancedActorsEditorModule::CreateLevelViewportContextMenuExtender(const TSharedRef<FUICommandList> CommandList, const TArray<AActor*> InActors)
{
	TSharedRef<FExtender> Extender = MakeShareable(new FExtender);
	if (InActors.Num() > 0)
	{
		const FText ActorName = InActors.Num() == 1 ? FText::Format(LOCTEXT("ActorNameSingular", "\"{0}\""), FText::FromString(InActors[0]->GetActorLabel())) : LOCTEXT("ActorNamePlural", "Actors");

		FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>(TEXT("LevelEditor"));
		TSharedRef<FUICommandList> LevelEditorCommandBindings = LevelEditor.GetGlobalLevelEditorActions();

		// Note: ActorConvert extension point appears only in the pulldown Actor menu.
		Extender->AddMenuExtension("ActorConvert", EExtensionHook::After, LevelEditorCommandBindings, FMenuExtensionDelegate::CreateLambda(
			[this, ActorName, InActors](FMenuBuilder& MenuBuilder)
			{
				bool bCanExecuteActorToIA = false;
				bool bCanExecuteIAtoActor = false;
				
				// we can stop checking as soon as we know we have both cases in Selected Actors (the InActors array).
				for (int32 Index = 0; Index < InActors.Num() && !(bCanExecuteActorToIA && bCanExecuteIAtoActor); ++Index)
				{
					const bool bIsIA = InActors[Index]->GetClass()->IsChildOf(AInstancedActorsManager::StaticClass());
					// We can only convert and Actor to an IA if it's not an IA instance
					bCanExecuteActorToIA = bCanExecuteActorToIA || !bIsIA;
					// We can only convert instances to and Actors only if it _is_ an IA
					bCanExecuteIAtoActor = bCanExecuteIAtoActor || bIsIA;
				}

				MenuBuilder.AddMenuEntry(
					FText::Format(ActorToIAFormatLabel, ActorName),
					LOCTEXT("ConvertSelectedActorsToIAsTooltip", "Convert the selected actors to Instanced Actors instances."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Convert"),
					FUIAction(
						FExecuteAction::CreateLambda([this, InActors]()
							{
								ActorToIADelegate.ExecuteIfBound(InActors);
							}),
						FCanExecuteAction::CreateLambda([bCanExecuteActorToIA]() { return bCanExecuteActorToIA; }))
				);

				MenuBuilder.AddMenuEntry(
					FText::Format(IAToActorFormatLabel, ActorName),
					LOCTEXT("ConvertSelectedIAsToActorsToolTip", "Convert all the Instanced Actors instances back to Actors."),
					FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Convert"),
					FUIAction(
						FExecuteAction::CreateLambda([this, InActors]()
							{
								IAToActorDelegate.ExecuteIfBound(InActors);
							}),
						FCanExecuteAction::CreateLambda([bCanExecuteIAtoActor]() { return bCanExecuteIAtoActor; }))
				);
			})
		);
	}
	return Extender;
}

void FInstancedActorsEditorModule::ConvertActorsToIAsUIAction(TConstArrayView<AActor*> InActors) const
{
	CustomizedConvertActorsToIAsUIAction(InActors, GET_INSTANCEDACTORS_CONFIG_VALUE(GetInstancedActorsSubsystemClass()));
}

void FInstancedActorsEditorModule::CustomizedConvertActorsToIAsUIAction(TConstArrayView<AActor*> InActors
	, TSubclassOf<UInstancedActorsSubsystem> IASubsystemClass) const
{
	check(GEditor);

	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		UE_LOGF(LogInstancedActorsEditor, Log, "Unable to get Editor world.");
		return;
	}

	UInstancedActorsSubsystem* IASubsystem = Cast<UInstancedActorsSubsystem>(World->GetSubsystemBase(IASubsystemClass));
	check(IASubsystem);

	FScopedTransaction Transaction(LOCTEXT("ConvertToIA_Transaction", "Convert Actors to IAs"));
	GEditor->SelectNone(/*bNoteSelectionChange=*/true, /*bDeselectBSPSurfs=*/true, /*WarnAboutManyActors=*/false);

	for (AActor* Actor : InActors)
	{
		// note that we skip all the IAs here, we don't support converting IA to instances.
		// We can end up here if there are multiple different AActors selected and some of them are IAs - the option
		// to convert will still appear in the "Actor" menu.
		if (Actor == nullptr || Actor->GetClass()->IsChildOf(AInstancedActorsManager::StaticClass()))
		{
			continue;
		}

		FInstancedActorsInstanceHandle InstanceHandle = IASubsystem->InstanceActor(Actor->GetClass(), Actor->GetActorTransform(), World->GetCurrentLevel());
		if (InstanceHandle.IsValid())
		{
			Actor->Destroy(); // This will call modify too.
			InstanceHandle.GetManager()->Modify();
			GEditor->SelectActor(InstanceHandle.GetManager(), /*bInSelected*/ true, /*bNotify*/ true, /*bSelectEvenIfHidden*/ true);
		}
	}
}

void FInstancedActorsEditorModule::ConvertIAsToActorsUIAction(TConstArrayView<AActor*> InActors) const
{
	check(GEditor);

	UWorld* World = GEditor->GetEditorWorldContext().World();
	if (!World)
	{
		UE_LOGF(LogInstancedActorsEditor, Log, "Unable to get Editor world.");
		return;
	}

	UInstancedActorsSubsystem* IASubsystem = World->GetSubsystem<UInstancedActorsSubsystem>();
	check(IASubsystem);

	FScopedTransaction Transaction(LOCTEXT("ConvertToActorsFromIA_Transaction", "Convert IAs to Actors"));
	GEditor->SelectNone(/*bNoteSelectionChange=*/true, /*bDeselectBSPSurfs=*/true, /*WarnAboutManyActors=*/false);

	FActorSpawnParameters SpawnParams;
	SpawnParams.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
	SpawnParams.OverrideLevel = World->GetCurrentLevel();

	for (AActor* Actor : InActors)
	{
		if (AInstancedActorsManager* Manager = Cast<AInstancedActorsManager>(Actor))
		{
			Manager->Modify();
			FInstancedActorsIterationContext IterationContext;
			Manager->ForEachInstance(
				[&SpawnParams](const FInstancedActorsInstanceHandle& InstanceHandle, const FTransform& InstanceTransform, FInstancedActorsIterationContext& IterationContext)
				{
					AActor* SpawnedActor = nullptr;

					const TSubclassOf<AActor>& ActorClass = InstanceHandle.GetInstanceActorDataChecked().ActorClass;
					// Start by trying to use the ActorFactory.
					if (UActorFactory* ActorFactory = GEditor->FindActorFactoryForActorClass(ActorClass))
					{
						SpawnedActor = ActorFactory->CreateActor(ActorClass, SpawnParams.OverrideLevel, InstanceTransform, SpawnParams);
					}
					// No factory found, fallback.
					else 
					{
						SpawnedActor = GEditor->AddActor(SpawnParams.OverrideLevel, ActorClass, InstanceTransform, /*bSilent*/true);
					}

					if (SpawnedActor)
					{
						const FVector Scale3D = InstanceTransform.GetScale3D();
						if (FVector::DistSquared(Scale3D, FVector::One()) > KINDA_SMALL_NUMBER)
						{
							SpawnedActor->SetActorScale3D(Scale3D);
						}
						GEditor->SelectActor(SpawnedActor, /*bInSelected*/ true, /*bNotify*/ true, /*bSelectEvenIfHidden*/ true);
						InstanceHandle.GetManagerChecked().RemoveActorInstance(InstanceHandle);
					}

					// Continue
					return true;
				}
				, IterationContext);

			// since we removed all content from the instance we can just as well destroy it. 
			Manager->Destroy();
		}
	}
}

void FInstancedActorsEditorModule::SetIAToActorDelegate(const FOnConvert& InDelegate, const FTextFormat& ActionFormatLabelOverride)
{
	IAToActorDelegate = InDelegate;
	IAToActorFormatLabel = ActionFormatLabelOverride;
}

void FInstancedActorsEditorModule::SetActorToIADelegate(const FOnConvert& InDelegate, const FTextFormat& ActionFormatLabelOverride)
{
	ActorToIADelegate = InDelegate;
	ActorToIAFormatLabel = ActionFormatLabelOverride;
}

#undef LOCTEXT_NAMESPACE
