// Copyright Epic Games, Inc. All Rights Reserved.

#include "InstancedActorsEditorSubsystem.h"
#include "InstancedActorsIndex.h"
#include "InstancedActorsData.h"

FInstancedActorsInstanceHandle UInstancedActorsEditorSubsystem::GetInstanceHandle(UInstancedActorsData* InstanceData, int32 Index)
{
	if (!InstanceData || Index < 0 || Index >= InstanceData->InstanceTransforms.Num())
	{
		UE_LOGF(LogInstancedActors, Error, "Invalid data, returning default handle.");
		return FInstancedActorsInstanceHandle();
	}

	const FInstancedActorsInstanceHandle Handle(*InstanceData, FInstancedActorsInstanceIndex(Index));
	return Handle;
}
