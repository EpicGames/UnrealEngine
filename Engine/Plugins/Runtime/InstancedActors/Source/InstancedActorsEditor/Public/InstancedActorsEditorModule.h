// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Templates/SharedPointer.h"
#include "Delegates/IDelegateInstance.h"
#include "Modules/ModuleInterface.h"
#include "Templates/SubclassOf.h"

#define UE_API INSTANCEDACTORSEDITOR_API

class AActor;
class FExtender;
class FUICommandList;
class AInstancedActorsManager;
class UInstancedActorsSubsystem;

/**
* The public interface to this module
* This module contains logic for converting between AActors and Instanced Actors. 
* @todo move to a dedicated class
*/
class FInstancedActorsEditorModule : public IModuleInterface
{
public:
	DECLARE_DELEGATE_OneParam(FOnConvert, TConstArrayView<AActor*>);

	/** Convert selected actors to Instance Actors (IAs) while using the IASubsystemClass */
	UE_API void CustomizedConvertActorsToIAsUIAction(TConstArrayView<AActor*> InActors, TSubclassOf<UInstancedActorsSubsystem> IASubsystemClass) const;
	
	/** 
	 * Sets InDelegate as the delegate that will be executed as "Convert Instanced Actors to regular Actors" action.
	 * @param ActionFormatLabelOverride if supplied will be used to override the default action label. Note that 
	 *	this is a format string and needs to require one parameter, the Actor's name
	 */
	UE_API void SetIAToActorDelegate(const FOnConvert& InDelegate, const FTextFormat& ActionFormatLabelOverride = FTextFormat());

	/**
	 * Sets InDelegate as the delegate that will be executed as "Convert Actors to Instanced Actors" action.
	 * @param ActionFormatLabelOverride if supplied will be used to override the default action label. Note that
	 *	this is a format string and needs to require one parameter, the Actor's name
	 */
	UE_API void SetActorToIADelegate(const FOnConvert& InDelegate, const FTextFormat& ActionFormatLabelOverride = FTextFormat());

	/** Resets FOnConvert delegates and action labels to their defaults */
	UE_API void ResetConversionDelegates();

protected:
	// Begin IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;

	/** Convert selected actors to Instance Actors (IAs). */
	UE_API void ConvertActorsToIAsUIAction(TConstArrayView<AActor*> InActors) const;

	/** Convert selected Instance Actors to Actors. */
	UE_API void ConvertIAsToActorsUIAction(TConstArrayView<AActor*> InActors) const;

	UE_API TSharedRef<FExtender> CreateLevelViewportContextMenuExtender(const TSharedRef<FUICommandList> CommandList, const TArray<AActor*> InActors);
	UE_API void AddLevelViewportMenuExtender();
	UE_API void RemoveLevelViewportMenuExtender();

	FDelegateHandle LevelViewportExtenderHandle;
	FOnConvert ActorToIADelegate;
	FOnConvert IAToActorDelegate;
	FText CustomizedLabelPrefix;
	FTextFormat ActorToIAFormatLabel;
	FTextFormat IAToActorFormatLabel;
};

#undef UE_API
