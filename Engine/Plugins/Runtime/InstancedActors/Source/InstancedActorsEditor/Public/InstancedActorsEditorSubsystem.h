// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorSubsystem.h"
#include "InstancedActorsIndex.h"

#include "InstancedActorsEditorSubsystem.generated.h"

#define UE_API INSTANCEDACTORSEDITOR_API

class UInstancedActorsData;

/**
* Subsystem for exposing instanced actors functionality to blueprints
*/
UCLASS(MinimalAPI)
class UInstancedActorsEditorSubsystem : public UEditorSubsystem
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintPure, Category = "InstancedActors", meta = (DisplayName = "Get Instance Handle (by Index)"))
	static UE_API FInstancedActorsInstanceHandle GetInstanceHandle(UInstancedActorsData* InstanceData, int32 Index);
};

#undef UE_API
