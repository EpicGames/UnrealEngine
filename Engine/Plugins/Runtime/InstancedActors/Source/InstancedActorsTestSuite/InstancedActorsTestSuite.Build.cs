// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class InstancedActorsTestSuite : ModuleRules
	{
		public InstancedActorsTestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"InstancedActors",
					"AITestSuite",
				}
			);
		}
	}
}