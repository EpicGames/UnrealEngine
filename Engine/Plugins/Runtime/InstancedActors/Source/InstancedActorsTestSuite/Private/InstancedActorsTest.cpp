// Copyright Epic Games, Inc. All Rights Reserved.

#include "AITestsCommon.h"

#define LOCTEXT_NAMESPACE "InstancedActorsTest"

UE_DISABLE_OPTIMIZATION_SHIP

UE_ENABLE_OPTIMIZATION_SHIP

#undef LOCTEXT_NAMESPACE
